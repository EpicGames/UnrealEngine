// Copyright Epic Games, Inc. All Rights Reserved.

#include "Backends/CborStructDeserializerBackend.h"

#include "Backends/StructDeserializerBackendUtilities.h"
#include "StructSerializationUtilities.h"
#include "UObject/Class.h"
#include "UObject/AnsiStrProperty.h"
#include "UObject/EnumProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"
#include "UObject/Utf8StrProperty.h"


FCborStructDeserializerBackend::FCborStructDeserializerBackend(FArchive& Archive, ECborEndianness CborDataEndianness, bool bInIsLWCCompatibilityMode)
	: CborReader(&Archive, CborDataEndianness)
	, bIsLWCCompatibilityMode(bInIsLWCCompatibilityMode)
{}

FCborStructDeserializerBackend::~FCborStructDeserializerBackend() = default;

const FString& FCborStructDeserializerBackend::GetCurrentPropertyName() const
{
	return LastMapKey;
}

FString FCborStructDeserializerBackend::GetDebugString() const
{
	FArchive* Ar = const_cast<FArchive*>(CborReader.GetArchive());
	return FString::Printf(TEXT("Offset: %" UINT64_FMT), Ar ? Ar->Tell() : 0);
}

const FString& FCborStructDeserializerBackend::GetLastErrorMessage() const
{
	// interface function that is actually entirely unused...
	static FString Dummy;
	return Dummy;
}

namespace UE::CborStructDeserializerBackend::Private
{

FString ConvertCborTypeToString(ECborCode CborCode)
{
	switch (CborCode)
	{
		case ECborCode::Uint:
			return TEXT("Uint");
		case ECborCode::Int:
			return TEXT("Int");
		case ECborCode::ByteString:
			return TEXT("ByteString");
		case ECborCode::TextString:
			return TEXT("TextString");						
		case ECborCode::Array:
			return TEXT("Array");
		case ECborCode::Map:
			return TEXT("Map");
		case ECborCode::Tag:
			return TEXT("Tag");
		case ECborCode::Prim:
			return TEXT("Prim");
		// Fallthrough
	};
	return TEXT("InvalidType");
}

} // namespace UE::CborStructDeserializerBackend::Private

bool FCborStructDeserializerBackend::GetNextToken(EStructDeserializerBackendTokens& OutToken)
{
	LastMapKey.Reset();

	if (bDeserializingByteArray) // Deserializing the content of a TArray<uint8>/TArray<int8> property?
	{
		if (DeserializingByteArrayIndex < LastContext.AsByteArray().Num())
		{
			OutToken = EStructDeserializerBackendTokens::Property; // Need to consume a byte from the CBOR ByteString as a UByteProperty/UInt8Property.
		}
		else
		{
			bDeserializingByteArray = false;
			OutToken = EStructDeserializerBackendTokens::ArrayEnd; // All bytes from the byte string were deserialized into the TArray<uint8>/TArray<int8>.
		}

		return true;
	}

	if (!CborReader.ReadNext(LastContext))
	{
		OutToken = LastContext.IsError() ? EStructDeserializerBackendTokens::Error : EStructDeserializerBackendTokens::None;
		return false;
	}

	if (LastContext.IsBreak())
	{
		ECborCode ContainerEndType = LastContext.AsBreak();
		// We do not support indefinite string container type
		check(ContainerEndType == ECborCode::Array || ContainerEndType == ECborCode::Map);
		OutToken = ContainerEndType == ECborCode::Array ? EStructDeserializerBackendTokens::ArrayEnd : EStructDeserializerBackendTokens::StructureEnd;
		return true;
	}

	// We've read an odd number of items from this map so far. So the next value is the key and it must be a string.
	if (CborReader.GetContext().MajorType() == ECborCode::Map && (CborReader.GetContext().AsLength() & 1))
	{
		// This code is designed to deserialize CBOR data into Unreal Engine structs. If this is not a string then we are somehow not dealing with UE
		// structs and this would be an error. Previously this code would just assert and crash.  This gracefully handles this case and passes the
		// failure down to the caller. 		
		if (LastContext.MajorType() == ECborCode::TextString)
		{
			LastMapKey = LastContext.AsString();

			// Read next and carry on
			if (!CborReader.ReadNext(LastContext))
			{
				OutToken = LastContext.IsError() ? EStructDeserializerBackendTokens::Error : EStructDeserializerBackendTokens::None;
				return false;
			}
		}
		else
		{
			OutToken = EStructDeserializerBackendTokens::Error;
			FString CborMajorTypeAsString = UE::CborStructDeserializerBackend::Private::ConvertCborTypeToString(LastContext.MajorType());
			uint8 CborMajorTypeAsInt = static_cast<uint8>(LastContext.MajorType());
			UE_LOGF(LogSerialization, Error, "Encountered invalided context while parsing CBOR map. Expected a string type but got code %d (%ls)", CborMajorTypeAsInt, *CborMajorTypeAsString);
			return false;
		}
	}

	switch (LastContext.MajorType())
	{
	case ECborCode::Array:
		OutToken = EStructDeserializerBackendTokens::ArrayStart;
		break;
	case ECborCode::Map:
		OutToken = EStructDeserializerBackendTokens::StructureStart;
		break;
	case ECborCode::ByteString: // Used for size optimization on TArray<uint8>/TArray<int8>. Might be replaced if https://datatracker.ietf.org/doc/draft-ietf-cbor-array-tags/ is adopted.
		OutToken = EStructDeserializerBackendTokens::ArrayStart;
		DeserializingByteArrayIndex = 0;
		bDeserializingByteArray = true;
		break;
	case ECborCode::Int:
		// fall through
	case ECborCode::Uint:
		// fall through
	case ECborCode::TextString:
		// fall through
	case ECborCode::Prim:
		OutToken = EStructDeserializerBackendTokens::Property;
		break;
	default:
		// Other types are unsupported
		check(false);
	}

	return true;
}

bool FCborStructDeserializerBackend::ReadProperty(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex)
{
	switch (LastContext.MajorType())
	{
	// Unsigned Integers
	case ECborCode::Uint:
	{
		if (FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(ByteProperty, Outer, Data, ArrayIndex, (uint8)LastContext.AsUInt());
		}

		if (FUInt16Property* FInt16Property = CastField<FUInt16Property>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(FInt16Property, Outer, Data, ArrayIndex, (uint16)LastContext.AsUInt());
		}

		if (FUInt32Property* UInt32Property = CastField<FUInt32Property>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(UInt32Property, Outer, Data, ArrayIndex, (uint32)LastContext.AsUInt());
		}

		if (FUInt64Property* FInt64Property = CastField<FUInt64Property>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(FInt64Property, Outer, Data, ArrayIndex, (uint64)LastContext.AsUInt());
		}
	}
	// Fall through - cbor can encode positive signed integers as unsigned
	// Signed Integers
	case ECborCode::Int:
	{
		if (FInt8Property* Int8Property = CastField<FInt8Property>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(Int8Property, Outer, Data, ArrayIndex, (int8)LastContext.AsInt());
		}

		if (FInt16Property* Int16Property = CastField<FInt16Property>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(Int16Property, Outer, Data, ArrayIndex, (int16)LastContext.AsInt());
		}

		if (FIntProperty* IntProperty = CastField<FIntProperty>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(IntProperty, Outer, Data, ArrayIndex, (int32)LastContext.AsInt());
		}

		if (FInt64Property* Int64Property = CastField<FInt64Property>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(Int64Property, Outer, Data, ArrayIndex, (int64)LastContext.AsInt());
		}

		UE_LOG(LogSerialization, Verbose, TEXT("Integer field %s with value '%" UINT64_FMT "' is not supported in FProperty type %s (%s)"), *Property->GetFName().ToString(), LastContext.AsUInt(), *Property->GetClass()->GetName(), *GetDebugString());

		return false;
	}

	// Strings, Names, Enumerations & Object/Class reference
	case ECborCode::TextString:
	{
		if (FAnsiStrProperty* AnsiStrProperty = CastField<FAnsiStrProperty>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(AnsiStrProperty, Outer, Data, ArrayIndex, LastContext.AsAnsiString());
		}

		if (FUtf8StrProperty* Utf8StrProperty = CastField<FUtf8StrProperty>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(Utf8StrProperty, Outer, Data, ArrayIndex, LastContext.AsUtf8String());
		}

		FString StringValue = LastContext.AsString();

		if (FStrProperty* StrProperty = CastField<FStrProperty>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(StrProperty, Outer, Data, ArrayIndex, StringValue);
		}

		if (FNameProperty* NameProperty = CastField<FNameProperty>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(NameProperty, Outer, Data, ArrayIndex, FName(*StringValue));
		}

		if (FTextProperty* TextProperty = CastField<FTextProperty>(Property))
		{
			FText TextValue;
			if (!FTextStringHelper::ReadFromBuffer(*StringValue, TextValue))
			{
				TextValue = FText::FromString(StringValue);
			}
			return StructDeserializerBackendUtilities::SetPropertyValue(TextProperty, Outer, Data, ArrayIndex, TextValue);
		}

		if (FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
		{
			if (!ByteProperty->Enum)
			{
				return false;
			}

			int64 Value = ByteProperty->Enum->GetValueByName(*StringValue);
			if (Value == INDEX_NONE)
			{
				return false;
			}

			return StructDeserializerBackendUtilities::SetPropertyValue(ByteProperty, Outer, Data, ArrayIndex, (uint8)Value);
		}

		if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			int64 Value = EnumProperty->GetEnum()->GetValueByName(*StringValue);
			if (Value == INDEX_NONE)
			{
				return false;
			}

			if (void* ElementPtr = StructDeserializerBackendUtilities::GetPropertyValuePtr(EnumProperty, Outer, Data, ArrayIndex))
			{
				EnumProperty->GetUnderlyingProperty()->SetIntPropertyValue(ElementPtr, Value);
				return true;
			}

			return false;
		}

		if (FClassProperty* ClassProperty = CastField<FClassProperty>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(ClassProperty, Outer, Data, ArrayIndex, LoadObject<UClass>(NULL, *StringValue, NULL, LOAD_NoWarn));
		}

		if (FSoftClassProperty* SoftClassProperty = CastField<FSoftClassProperty>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(SoftClassProperty, Outer, Data, ArrayIndex, FSoftObjectPtr(FSoftObjectPath(StringValue)));
		}

		if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(ObjectProperty, Outer, Data, ArrayIndex, StaticFindObject(ObjectProperty->PropertyClass, nullptr, *StringValue));
		}

		if (FWeakObjectProperty* WeakObjectProperty = CastField<FWeakObjectProperty>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(WeakObjectProperty, Outer, Data, ArrayIndex, FWeakObjectPtr(StaticFindObject(WeakObjectProperty->PropertyClass, nullptr, *StringValue)));
		}

		if (FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(SoftObjectProperty, Outer, Data, ArrayIndex, FSoftObjectPtr(FSoftObjectPath(StringValue)));
		}

		UE_LOGF(LogSerialization, Verbose, "String field %ls with value '%ls' is not supported in FProperty type %ls (%ls)", *Property->GetFName().ToString(), *StringValue, *Property->GetClass()->GetName(), *GetDebugString());

		return false;
	}

	// Stream of bytes: Used for TArray<uint8>/TArray<int8>
	case ECborCode::ByteString:
	{
		check(bDeserializingByteArray);

		// Consume one byte from the byte string.
		TArrayView<const uint8> DeserializedByteArray = LastContext.AsByteArray();
		check(DeserializingByteArrayIndex < DeserializedByteArray.Num());
		uint8 ByteValue = DeserializedByteArray[DeserializingByteArrayIndex++];

		if (FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(ByteProperty, Outer, Data, ArrayIndex, ByteValue);
		}
		else if (FInt8Property* Int8Property = CastField<FInt8Property>(Property))
		{
			return StructDeserializerBackendUtilities::SetPropertyValue(Int8Property, Outer, Data, ArrayIndex, (int8)ByteValue);
		}

		UE_LOGF(LogSerialization, Verbose, "Error while deserializing field %ls. Unexpected UProperty type %ls. Expected a UByteProperty/UInt8Property to deserialize a TArray<uint8>/TArray<int8>", *Property->GetFName().ToString(), *Property->GetClass()->GetName());
		return false;
	}

	// Prim
	case ECborCode::Prim:
	{
		switch (LastContext.AdditionalValue())
		{
			// Boolean
		case ECborCode::True:
			// fall through
		case ECborCode::False:
		{
			const FCoreTexts& CoreTexts = FCoreTexts::Get();

			if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
			{
				return StructDeserializerBackendUtilities::SetPropertyValue(BoolProperty, Outer, Data, ArrayIndex, LastContext.AsBool());
			}
			UE_LOGF(LogSerialization, Verbose, "Boolean field %ls with value '%ls' is not supported in FProperty type %ls (%ls)", *Property->GetFName().ToString(), LastContext.AsBool() ? *(CoreTexts.True.ToString()) : *(CoreTexts.False.ToString()), *Property->GetClass()->GetName(), *GetDebugString());
			return false;
		}
			// Null
		case ECborCode::Null:
			return StructDeserializerBackendUtilities::ClearPropertyValue(Property, Outer, Data, ArrayIndex);
			// Float
		case ECborCode::Value_4Bytes:

			if (bIsLWCCompatibilityMode == true && StructSerializationUtilities::IsLWCType(Property->GetOwnerStruct()))
			{
				FDoubleProperty* DoubleProperty = CastField<FDoubleProperty>(Property);
				if (ensureMsgf(DoubleProperty, TEXT("Float field %s with value '%f' from LWC struct type '%s' was expected to be a DoubleProperty but was of type %s (%s)"), *Property->GetFName().ToString(), LastContext.AsFloat(), *Property->GetOwnerStruct()->GetName(), * Property->GetClass()->GetName(), *GetDebugString()))
				{
					const double DoubleValue = LastContext.AsFloat();
					return StructDeserializerBackendUtilities::SetPropertyValue(DoubleProperty, Outer, Data, ArrayIndex, DoubleValue);
				}
			}
			else if(FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property))
			{
				return StructDeserializerBackendUtilities::SetPropertyValue(FloatProperty, Outer, Data, ArrayIndex, LastContext.AsFloat());
			}
			
			UE_LOGF(LogSerialization, Verbose, "Float field %ls with value '%f' is not supported in FProperty type %ls (%ls)", *Property->GetFName().ToString(), LastContext.AsFloat(), *Property->GetClass()->GetName(), *GetDebugString());
			return false;
			// Double
		case ECborCode::Value_8Bytes:
			if (FDoubleProperty* DoubleProperty = CastField<FDoubleProperty>(Property))
			{
				return StructDeserializerBackendUtilities::SetPropertyValue(DoubleProperty, Outer, Data, ArrayIndex, LastContext.AsDouble());
			}
			UE_LOGF(LogSerialization, Verbose, "Double field %ls with value '%f' is not supported in FProperty type %ls (%ls)", *Property->GetFName().ToString(), LastContext.AsDouble(), *Property->GetClass()->GetName(), *GetDebugString());
			return false;
		default:
			UE_LOGF(LogSerialization, Verbose, "Unsupported primitive type for %ls with value '%f' in FProperty type %ls (%ls)", *Property->GetFName().ToString(), LastContext.AsDouble(), *Property->GetClass()->GetName(), *GetDebugString());
			return false;
		}
	}
	}

	return true;
}

bool FCborStructDeserializerBackend::ReadPODArray(FArrayProperty* ArrayProperty, void* Data)
{
	// if we just read a byte array, copy the full array if the inner property is of the appropriate type 
	if (bDeserializingByteArray
		&& (CastField<FByteProperty>(ArrayProperty->Inner) || CastField<FInt8Property>(ArrayProperty->Inner)))
	{
		FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->template ContainerPtrToValuePtr<void>(Data));
		TArrayView<const uint8> DeserializedByteArray = LastContext.AsByteArray();
		if (DeserializedByteArray.Num())
		{
			ArrayHelper.AddUninitializedValues(DeserializedByteArray.Num());
			void* ArrayStart = ArrayHelper.GetRawPtr();
			FMemory::Memcpy(ArrayStart, DeserializedByteArray.GetData(), DeserializedByteArray.Num());
		}
		bDeserializingByteArray = false;
		return true;
	}
	return false;
}

void FCborStructDeserializerBackend::SkipArray()
{
	if (bDeserializingByteArray) // Deserializing a TArray<uint8>/TArray<int8> property as byte string?
	{
		check(DeserializingByteArrayIndex == 0);
		bDeserializingByteArray = false;
	}
	else
	{
		CborReader.SkipContainer(ECborCode::Array);
	}
}

void FCborStructDeserializerBackend::SkipStructure()
{
	CborReader.SkipContainer(ECborCode::Map);
}
