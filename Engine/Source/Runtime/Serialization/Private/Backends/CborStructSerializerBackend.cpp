// Copyright Epic Games, Inc. All Rights Reserved.

#include "Backends/CborStructSerializerBackend.h"

#include "StructSerializationUtilities.h"
#include "UObject/AnsiStrProperty.h"
#include "UObject/EnumProperty.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/StrProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"
#include "UObject/Utf8StrProperty.h"

FCborStructSerializerBackend::FCborStructSerializerBackend(FArchive& InArchive, const EStructSerializerBackendFlags InFlags)
	: CborWriter(&InArchive, EnumHasAnyFlags(InFlags, EStructSerializerBackendFlags::WriteCborStandardEndianness) ? ECborEndianness::StandardCompliant : ECborEndianness::Platform)
	, Flags(InFlags)
{}

FCborStructSerializerBackend::~FCborStructSerializerBackend() = default;

void FCborStructSerializerBackend::BeginArray(const FStructSerializerState& State)
{
	// If TArray<uint8>/TArray<int8> content needs to be written as ByteString (to prevent paying a 1 byte header for each byte greater than 23 required by CBOR array).
	if (EnumHasAnyFlags(Flags, EStructSerializerBackendFlags::WriteByteArrayAsByteStream))
	{
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(State.ValueProperty))
		{
			if (CastField<FByteProperty>(ArrayProperty->Inner) || CastField<FInt8Property>(ArrayProperty->Inner)) // A CBOR draft to support homogeneous array exists, but is not yet approved: https://datatracker.ietf.org/doc/draft-ietf-cbor-array-tags/.
			{
				check(!bSerializingByteArray);
				// WritePODArray should be preferred in this case instead of doing per element serialization
				// Hence omit reserving the space needed for per element serialization since it will be written directly to the cbor stream
				// if otherwise per element serialization is done, it will still work although through some additional allocation
				//FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(State.ValueData));
				//AccumulatedBytes.Reset(ArrayHelper.Num());				
				bSerializingByteArray = true;
			}
		}
	}

	// Array nested in Array/Set
	if (State.ValueProperty->GetOwner<FArrayProperty>() || State.ValueProperty->GetOwner<FSetProperty>())
	{
		// fall through.
	}
	// Array nested in Map
	else if (State.KeyProperty != nullptr)
	{
		FString KeyString;
		State.KeyProperty->ExportTextItem_Direct(KeyString, State.KeyData, nullptr, nullptr, PPF_None);
		CborWriter.WriteValue(KeyString);
	}
	// Array nested in Object
	else
	{
		CborWriter.WriteValue(State.ValueProperty->GetName());
	}

	if (!bSerializingByteArray) // TArray<uint8>/TArray<int8> are written as ByteString rather than CBOR array because it is more size efficient.
	{
		CborWriter.WriteContainerStart(ECborCode::Array, -1/*Indefinite*/);
	}
}

void FCborStructSerializerBackend::BeginStructure(const FStructSerializerState& State)
{
	if (State.ValueProperty != nullptr)
	{
		// Object nested in Array/Set
		if ((State.ValueProperty->ArrayDim > 1
			|| State.ValueProperty->GetOwner<FArrayProperty>()
			|| State.ValueProperty->GetOwner<FSetProperty>()
			|| (State.ValueProperty->GetOwner<FMapProperty>() && State.KeyProperty == nullptr)) && !EnumHasAnyFlags(State.StateFlags, EStructSerializerStateFlags::WritingContainerElement))
		{
			CborWriter.WriteContainerStart(ECborCode::Map, -1/*Indefinite*/);
		}
		// Object nested in Map
		else if (State.KeyProperty != nullptr)
		{
			FString KeyString;
			State.KeyProperty->ExportTextItem_Direct(KeyString, State.KeyData, nullptr, nullptr, PPF_None);
			CborWriter.WriteValue(KeyString);
			CborWriter.WriteContainerStart(ECborCode::Map, -1/*Indefinite*/);
		}
		// Object nested in Object
		else
		{
			CborWriter.WriteValue(State.ValueProperty->GetName());
			CborWriter.WriteContainerStart(ECborCode::Map, -1/*Indefinite*/);
		}
	}
	// Root Object
	else
	{
		CborWriter.WriteContainerStart(ECborCode::Map, -1/*Indefinite*/);
	}
}

void FCborStructSerializerBackend::EndArray(const FStructSerializerState& State)
{
	if (bSerializingByteArray) // Does end a TArray<uint8>/TArray<int8>?
	{
		// Flush the accumulated bytes as a ByteString().
		CborWriter.WriteValue(AccumulatedBytes.GetData(), AccumulatedBytes.Num());
		bSerializingByteArray = false;
	}
	else
	{
		CborWriter.WriteContainerEnd();
	}
}

void FCborStructSerializerBackend::EndStructure(const FStructSerializerState& State)
{
	CborWriter.WriteContainerEnd();
}

void FCborStructSerializerBackend::WriteComment(const FString& Comment)
{
	// Binary format do not support comment
}

namespace CborStructSerializerBackend
{
	// Writes a property value to the serialization output.
	template<typename ValueType>
	void WritePropertyValue(FCborWriter& CborWriter, const FStructSerializerState& State, const ValueType& Value)
	{
		// Value nested in Array/Set (except single element) or map as array or as root 
		if ((State.ValueProperty == nullptr) ||
			((State.ValueProperty->ArrayDim > 1
				|| State.ValueProperty->GetOwner<FArrayProperty>()
				|| State.ValueProperty->GetOwner<FSetProperty>()
				|| (State.ValueProperty->GetOwner<FMapProperty>() && State.KeyProperty == nullptr)) && !EnumHasAnyFlags(State.StateFlags, EStructSerializerStateFlags::WritingContainerElement)))
		{
			CborWriter.WriteValue(Value);
		}
		// Value nested in Map
		else if (State.KeyProperty != nullptr)
		{
			FString KeyString;
			State.KeyProperty->ExportTextItem_Direct(KeyString, State.KeyData, nullptr, nullptr, PPF_None);
			CborWriter.WriteValue(KeyString);
			CborWriter.WriteValue(Value);
		}
		// Value nested in Object
		else
		{
			CborWriter.WriteValue(State.ValueProperty->GetName());
			CborWriter.WriteValue(Value);
		}
	}

	// Writes a null value to the serialization output.
	void WriteNull(FCborWriter& CborWriter, const FStructSerializerState& State)
	{
		if ((State.ValueProperty == nullptr) ||
			((State.ValueProperty->ArrayDim > 1
				|| State.ValueProperty->GetOwner<FArrayProperty>()
				|| State.ValueProperty->GetOwner<FSetProperty>()
				|| (State.ValueProperty->GetOwner<FMapProperty>() && State.KeyProperty == nullptr)) && !EnumHasAnyFlags(State.StateFlags, EStructSerializerStateFlags::WritingContainerElement)))
		{
			CborWriter.WriteNull();
		}
		else if (State.KeyProperty != nullptr)
		{
			FString KeyString;
			State.KeyProperty->ExportTextItem_Direct(KeyString, State.KeyData, nullptr, nullptr, PPF_None);
			CborWriter.WriteValue(KeyString);
			CborWriter.WriteNull();
		}
		else
		{
			CborWriter.WriteValue(State.ValueProperty->GetName());
			CborWriter.WriteNull();
		}
	}
}

void FCborStructSerializerBackend::WriteProperty(const FStructSerializerState& State, int32 ArrayIndex)
{
	using namespace CborStructSerializerBackend;

	// Bool
	if (State.FieldType == FBoolProperty::StaticClass())
	{
		WritePropertyValue(CborWriter, State, CastFieldChecked<FBoolProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}

	// Unsigned Bytes & Enums
	else if (State.FieldType == FEnumProperty::StaticClass())
	{
		FEnumProperty* EnumProperty = CastFieldChecked<FEnumProperty>(State.ValueProperty);

		WritePropertyValue(CborWriter, State, EnumProperty->GetEnum()->GetNameStringByValue(EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(EnumProperty->ContainerPtrToValuePtr<void>(State.ValueData, ArrayIndex))));
	}
	else if (State.FieldType == FByteProperty::StaticClass())
	{
		FByteProperty* ByteProperty = CastFieldChecked<FByteProperty>(State.ValueProperty);

		if (ByteProperty->IsEnum())
		{
			WritePropertyValue(CborWriter, State, ByteProperty->Enum->GetNameStringByValue(ByteProperty->GetPropertyValue_InContainer(State.ValueData, ArrayIndex)));
		}
		else if (bSerializingByteArray) // Writing a byte from a TArray<uint8>/TArray<int8>?
		{
			AccumulatedBytes.Add(ByteProperty->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
		}
		else
		{
			WritePropertyValue(CborWriter, State, (int64)ByteProperty->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
		}
	}

	// Double & Float
	else if (State.FieldType == FDoubleProperty::StaticClass())
	{
		if (EnumHasAnyFlags(Flags, EStructSerializerBackendFlags::WriteLWCTypesAsFloats) && StructSerializationUtilities::IsLWCType(State.ValueProperty->GetOwnerStruct()))
		{
			const double Value = CastFieldChecked<FDoubleProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex);
			WritePropertyValue(CborWriter, State, static_cast<float>(Value));
		}
		else
		{
			WritePropertyValue(CborWriter, State, CastFieldChecked<FDoubleProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
		}
	}
	else if (State.FieldType == FFloatProperty::StaticClass())
	{
		WritePropertyValue(CborWriter, State, CastFieldChecked<FFloatProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}

	// Signed Integers
	else if (State.FieldType == FIntProperty::StaticClass())
	{
		WritePropertyValue(CborWriter, State, (int64)CastFieldChecked<FIntProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FInt8Property::StaticClass())
	{
		int8 Value = CastFieldChecked<FInt8Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex);
		if (bSerializingByteArray) // Writing a int8 from a TArray<uint8>/TArray<int8>?
		{
			AccumulatedBytes.Add(Value);
		}
		else
		{
			WritePropertyValue(CborWriter, State, (int64)Value);
		}
	}
	else if (State.FieldType == FInt16Property::StaticClass())
	{
		WritePropertyValue(CborWriter, State, (int64)CastFieldChecked<FInt16Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FInt64Property::StaticClass())
	{
		WritePropertyValue(CborWriter, State, (int64)CastFieldChecked<FInt64Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}

	// Unsigned Integers
	else if (State.FieldType == FUInt16Property::StaticClass())
	{
		WritePropertyValue(CborWriter, State, (int64)CastFieldChecked<FUInt16Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FUInt32Property::StaticClass())
	{
		WritePropertyValue(CborWriter, State, (int64)CastFieldChecked<FUInt32Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FUInt64Property::StaticClass())
	{
		WritePropertyValue(CborWriter, State, (int64)CastFieldChecked<FUInt64Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}

	// FNames, Strings & Text
	else if (State.FieldType == FNameProperty::StaticClass())
	{
		WritePropertyValue(CborWriter, State, CastFieldChecked<FNameProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex).ToString());
	}
	else if (State.FieldType == FStrProperty::StaticClass())
	{
		WritePropertyValue(CborWriter, State, CastFieldChecked<FStrProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FAnsiStrProperty::StaticClass())
	{
		WritePropertyValue(CborWriter, State, CastFieldChecked<FAnsiStrProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FUtf8StrProperty::StaticClass())
	{
		WritePropertyValue(CborWriter, State, CastFieldChecked<FUtf8StrProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FTextProperty::StaticClass())
	{
		const FText& TextValue = CastFieldChecked<FTextProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex);
		if (EnumHasAnyFlags(Flags, EStructSerializerBackendFlags::WriteTextAsComplexString))
		{
			FString TextValueString;
			FTextStringHelper::WriteToBuffer(TextValueString, TextValue);
			WritePropertyValue(CborWriter, State, TextValueString);
		}
		else
		{
			WritePropertyValue(CborWriter, State, TextValue.ToString());
		}
	}

	// Classes & Objects
	else if (State.FieldType == FSoftClassProperty::StaticClass())
	{
		FSoftObjectPtr const& Value = CastFieldChecked<FSoftClassProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex);
		WritePropertyValue(CborWriter, State, Value.ToString());
	}
	else if (State.FieldType == FWeakObjectProperty::StaticClass())
	{
		FWeakObjectPtr const& Value = CastFieldChecked<FWeakObjectProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex);
		WritePropertyValue(CborWriter, State, Value.IsValid() ? Value.Get()->GetPathName() : FString());
	}
	else if (State.FieldType == FSoftObjectProperty::StaticClass())
	{
		FSoftObjectPtr const& Value = CastFieldChecked<FSoftObjectProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex);
		WritePropertyValue(CborWriter, State, Value.ToString());
	}
	else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(State.ValueProperty))
	{
		// @TODO: Could this be expanded to include everything derived from FObjectPropertyBase?
		// Generic handling for a property type derived from FObjectProperty that is obtainable as a pointer and will be stored using its path.
		// This must come after all the more specialized handlers for object property types.
		UObject* const Value = ObjectProperty->GetObjectPropertyValue_InContainer(State.ValueData, ArrayIndex);
		WritePropertyValue(CborWriter, State, Value ? Value->GetPathName() : FString());
	}

	// Unsupported
	else
	{
		UE_LOGF(LogSerialization, Verbose, "FCborStructSerializerBackend: Property %ls cannot be serialized, because its type (%ls) is not supported", *State.ValueProperty->GetFName().ToString(), *State.ValueType->GetFName().ToString());
	}

}

bool FCborStructSerializerBackend::WritePODArray(const FStructSerializerState& State)
{
	FArrayProperty* ArrayProperty = CastField<FArrayProperty>(State.ValueProperty);
	if (bSerializingByteArray &&
		ArrayProperty && 
		(CastField<FByteProperty>(ArrayProperty->Inner) || CastField<FInt8Property>(ArrayProperty->Inner))) // A CBOR draft to support homogeneous array exists, but is not yet approved: https://datatracker.ietf.org/doc/draft-ietf-cbor-array-tags/.
	{
		FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(State.ValueData));
		// write out the array as a ByteString directly.
		CborWriter.WriteValue(ArrayHelper.GetRawPtr(), ArrayHelper.Num());
		bSerializingByteArray = false;
		return true;
	}
	return false;
}
