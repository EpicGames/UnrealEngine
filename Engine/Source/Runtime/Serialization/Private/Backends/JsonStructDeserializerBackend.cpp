// Copyright Epic Games, Inc. All Rights Reserved.

#include "Backends/JsonStructDeserializerBackend.h"
#include "Backends/StructDeserializerBackendUtilities.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"
#include "UObject/EnumProperty.h"
#include "UObject/TextProperty.h"

/* IStructDeserializerBackend interface
 *****************************************************************************/

const FString& FJsonStructDeserializerBackend::GetCurrentPropertyName() const
{
	return JsonReader->GetIdentifier();
}


FString FJsonStructDeserializerBackend::GetDebugString() const
{
	return FString::Printf(TEXT("Line: %u, Ch: %u"), JsonReader->GetLineNumber(), JsonReader->GetCharacterNumber());
}


const FString& FJsonStructDeserializerBackend::GetLastErrorMessage() const
{
	return JsonReader->GetErrorMessage();
}


bool FJsonStructDeserializerBackend::GetNextToken( EStructDeserializerBackendTokens& OutToken )
{
	if (!JsonReader->ReadNext(LastNotation))
	{
		return false;
	}

	switch (LastNotation)
	{
	case EJsonNotation::ArrayEnd:
		OutToken = EStructDeserializerBackendTokens::ArrayEnd;
		break;

	case EJsonNotation::ArrayStart:
		OutToken = EStructDeserializerBackendTokens::ArrayStart;
		break;

	case EJsonNotation::Boolean:
	case EJsonNotation::Null:
	case EJsonNotation::Number:
	case EJsonNotation::String:
		{
			OutToken = EStructDeserializerBackendTokens::Property;
		}
		break;

	case EJsonNotation::Error:
		OutToken = EStructDeserializerBackendTokens::Error;
		break;

	case EJsonNotation::ObjectEnd:
		OutToken = EStructDeserializerBackendTokens::StructureEnd;
		break;

	case EJsonNotation::ObjectStart:
		OutToken = EStructDeserializerBackendTokens::StructureStart;
		break;

	default:
		OutToken = EStructDeserializerBackendTokens::None;
	}

	return true;
}


bool FJsonStructDeserializerBackend::ReadProperty( FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex )
{
	switch (LastNotation)
	{
	// boolean values
	case EJsonNotation::Boolean:
		{
			bool BoolValue = JsonReader->GetValueAsBoolean();

			if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
			{
				return StructDeserializerBackendUtilities::SetPropertyValue(BoolProperty, Outer, Data, ArrayIndex, BoolValue);
			}

			const FCoreTexts& CoreTexts = FCoreTexts::Get();

			UE_LOGF(LogSerialization, Verbose, "Boolean field %ls with value '%ls' is not supported in FProperty type %ls (%ls)", *Property->GetFName().ToString(), BoolValue ? *(CoreTexts.True.ToString()) : *(CoreTexts.False.ToString()), *Property->GetClass()->GetName(), *GetDebugString());

			return false;
		}

	// numeric values
	case EJsonNotation::Number:
		{
			const FString& NumericValueString = JsonReader->GetValueAsNumberString();

			if (FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
			{
				int8 NumericValue = 0;
				if (LexTryParseString(NumericValue, *NumericValueString))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(ByteProperty, Outer, Data, ArrayIndex, NumericValue);
				}
			}

			if (FDoubleProperty* DoubleProperty = CastField<FDoubleProperty>(Property))
			{
				double NumericValue = 0;
				if (LexTryParseString(NumericValue, *NumericValueString))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(DoubleProperty, Outer, Data, ArrayIndex, NumericValue);
				}
			}

			if (FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property))
			{
				float NumericValue = 0;
				if (LexTryParseString(NumericValue, *NumericValueString))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(FloatProperty, Outer, Data, ArrayIndex, NumericValue);
				}
			}

			if (FIntProperty* IntProperty = CastField<FIntProperty>(Property))
			{
				int32 NumericValue = 0;
				if (LexTryParseString(NumericValue, *NumericValueString))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(IntProperty, Outer, Data, ArrayIndex, NumericValue);
				}
			}

			if (FUInt32Property* UInt32Property = CastField<FUInt32Property>(Property))
			{
				uint32 NumericValue = 0;
				if (LexTryParseString(NumericValue, *NumericValueString))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(UInt32Property, Outer, Data, ArrayIndex, NumericValue);
				}
			}

			if (FInt16Property* Int16Property = CastField<FInt16Property>(Property))
			{
				int16 NumericValue = 0;
				if (LexTryParseString(NumericValue, *NumericValueString))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(Int16Property, Outer, Data, ArrayIndex, NumericValue);
				}
			}

			if (FUInt16Property* FInt16Property = CastField<FUInt16Property>(Property))
			{
				uint16 NumericValue = 0;
				if (LexTryParseString(NumericValue, *NumericValueString))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(FInt16Property, Outer, Data, ArrayIndex, NumericValue);
				}
			}

			if (FInt64Property* Int64Property = CastField<FInt64Property>(Property))
			{
				int64 NumericValue = 0;
				if (LexTryParseString(NumericValue, *NumericValueString))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(Int64Property, Outer, Data, ArrayIndex, NumericValue);
				}
			}

			if (FUInt64Property* FInt64Property = CastField<FUInt64Property>(Property))
			{
				uint64 NumericValue = 0;
				if (LexTryParseString(NumericValue, *NumericValueString))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(FInt64Property, Outer, Data, ArrayIndex, NumericValue);
				}
			}

			if (FInt8Property* Int8Property = CastField<FInt8Property>(Property))
			{
				int8 NumericValue = 0;
				if (LexTryParseString(NumericValue, *NumericValueString))
				{
					return StructDeserializerBackendUtilities::SetPropertyValue(Int8Property, Outer, Data, ArrayIndex, NumericValue);
				}
			}

			UE_LOGF(LogSerialization, Verbose, "Numeric field %ls with value '%ls' is not supported in FProperty type %ls (%ls)", *Property->GetFName().ToString(), *NumericValueString, *Property->GetClass()->GetName(), *GetDebugString());

			return false;
		}

	// null values
	case EJsonNotation::Null:
		return StructDeserializerBackendUtilities::ClearPropertyValue(Property, Outer, Data, ArrayIndex);

	// strings, names, enumerations & object/class reference
	case EJsonNotation::String:
		{
			const FString& StringValue = JsonReader->GetValueAsString();

			if (FStrProperty* StrProperty = CastField<FStrProperty>(Property))
			{
				return StructDeserializerBackendUtilities::SetPropertyValue(StrProperty, Outer, Data, ArrayIndex, StringValue);
			}

			if (FNameProperty* NameProperty = CastField<FNameProperty>(Property))
			{
				return StructDeserializerBackendUtilities::SetPropertyValue(NameProperty, Outer, Data, ArrayIndex, FName(*StringValue));
			}

			if (FTextProperty* TextProperty = CastField<FTextProperty>(Property))
			{
				FText TextValue;
				if (!FTextStringHelper::ReadFromBuffer(*StringValue, TextValue))
				{
					TextValue = FText::FromString(StringValue);
				}
				return StructDeserializerBackendUtilities::SetPropertyValue(TextProperty, Outer, Data, ArrayIndex, TextValue);
			}

			if (FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
			{
				if (!ByteProperty->Enum)
				{
					return false;
				}

				int64 Value = ByteProperty->Enum->GetValueByName(*StringValue);
				if (Value == INDEX_NONE)
				{
					return false;
				}

				return StructDeserializerBackendUtilities::SetPropertyValue(ByteProperty, Outer, Data, ArrayIndex, (uint8)Value);
			}

			if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
			{
				int64 Value = EnumProperty->GetEnum()->GetValueByName(*StringValue);
				if (Value == INDEX_NONE)
				{
					return false;
				}

				if (void* ElementPtr = StructDeserializerBackendUtilities::GetPropertyValuePtr(EnumProperty, Outer, Data, ArrayIndex))
				{
					EnumProperty->GetUnderlyingProperty()->SetIntPropertyValue(ElementPtr, Value);
					return true;
				}

				return false;
			}

			if (FClassProperty* ClassProperty = CastField<FClassProperty>(Property))
			{
				return StructDeserializerBackendUtilities::SetPropertyValue(ClassProperty, Outer, Data, ArrayIndex, LoadObject<UClass>(nullptr, *StringValue, nullptr, LOAD_NoWarn));
			}

			if (FSoftClassProperty* SoftClassProperty = CastField<FSoftClassProperty>(Property))
			{
				return StructDeserializerBackendUtilities::SetPropertyValue(SoftClassProperty, Outer, Data, ArrayIndex, FSoftObjectPtr(LoadObject<UClass>(nullptr, *StringValue, nullptr, LOAD_NoWarn)));
			}

			if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
			{
				return StructDeserializerBackendUtilities::SetPropertyValue(ObjectProperty, Outer, Data, ArrayIndex, StaticFindObject(ObjectProperty->PropertyClass, nullptr, *StringValue));
			}

			if (FWeakObjectProperty* WeakObjectProperty = CastField<FWeakObjectProperty>(Property))
			{
				return StructDeserializerBackendUtilities::SetPropertyValue(WeakObjectProperty, Outer, Data, ArrayIndex, FWeakObjectPtr(StaticFindObject(WeakObjectProperty->PropertyClass, nullptr, *StringValue)));
			}

			if (FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(Property))
			{
				return StructDeserializerBackendUtilities::SetPropertyValue(SoftObjectProperty, Outer, Data, ArrayIndex, FSoftObjectPtr(FSoftObjectPath(StringValue)));
			}

			UE_LOGF(LogSerialization, Verbose, "String field %ls with value '%ls' is not supported in FProperty type %ls (%ls)", *Property->GetFName().ToString(), *StringValue, *Property->GetClass()->GetName(), *GetDebugString());

			return false;
		}
	}

	return true;
}


void FJsonStructDeserializerBackend::SkipArray()
{
	JsonReader->SkipArray();
}


void FJsonStructDeserializerBackend::SkipStructure()
{
	JsonReader->SkipObject();
}
