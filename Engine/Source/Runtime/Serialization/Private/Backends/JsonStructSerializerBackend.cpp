// Copyright Epic Games, Inc. All Rights Reserved.

#include "Backends/JsonStructSerializerBackend.h"
#include "UObject/StrProperty.h"
#include "UObject/UnrealType.h"
#include "UObject/EnumProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/PropertyPortFlags.h"


/* IStructSerializerBackend interface
 *****************************************************************************/

void FJsonStructSerializerBackend::BeginArray(const FStructSerializerState& State)
{
	if (State.ValueProperty->GetOwner<FArrayProperty>())
	{
		JsonWriter->WriteArrayStart();
	}
	else if (State.KeyProperty != nullptr)
	{
		FString KeyString;
		State.KeyProperty->ExportTextItem_Direct(KeyString, State.KeyData, nullptr, nullptr, PPF_None);
		JsonWriter->WriteArrayStart(KeyString);
	}
	else
	{
		JsonWriter->WriteArrayStart(State.ValueProperty->GetName());
	}
}


void FJsonStructSerializerBackend::BeginStructure(const FStructSerializerState& State)
{
	if (State.ValueProperty != nullptr)
	{
		//Write only object start in case of struct contained in arrays and not a single element is targeted
		if ((State.ValueProperty->ArrayDim > 1
			|| State.ValueProperty->GetOwner<FArrayProperty>()
			|| State.ValueProperty->GetOwner<FSetProperty>()
			|| (State.ValueProperty->GetOwner<FMapProperty>() && State.KeyProperty == nullptr)) && !EnumHasAnyFlags(State.StateFlags, EStructSerializerStateFlags::WritingContainerElement))
		{
			JsonWriter->WriteObjectStart();
		}
		else if (State.KeyProperty != nullptr)
		{
			FString KeyString;
			State.KeyProperty->ExportTextItem_Direct(KeyString, State.KeyData, nullptr, nullptr, PPF_None);
			JsonWriter->WriteObjectStart(KeyString);
		}
		else
		{
			JsonWriter->WriteObjectStart(State.ValueProperty->GetName());
		}
	}
	else
	{
		JsonWriter->WriteObjectStart();
	}
}


void FJsonStructSerializerBackend::EndArray(const FStructSerializerState& /*State*/)
{
	JsonWriter->WriteArrayEnd();
}


void FJsonStructSerializerBackend::EndStructure(const FStructSerializerState& /*State*/)
{
	JsonWriter->WriteObjectEnd();
}


void FJsonStructSerializerBackend::WriteComment(const FString& Comment)
{
	// Json does not support comments
}


void FJsonStructSerializerBackend::WriteProperty(const FStructSerializerState& State, int32 ArrayIndex)
{
	// booleans
	if (State.FieldType == FBoolProperty::StaticClass())
	{
		WritePropertyValue(State, CastFieldChecked<FBoolProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}

	// unsigned bytes & enumerations
	else if (State.FieldType == FEnumProperty::StaticClass())
	{
		FEnumProperty* EnumProperty = CastFieldChecked<FEnumProperty>(State.ValueProperty);

		WritePropertyValue(State, EnumProperty->GetEnum()->GetNameStringByValue(EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(EnumProperty->ContainerPtrToValuePtr<void>(State.ValueData, ArrayIndex))));
	}
	else if (State.FieldType == FByteProperty::StaticClass())
	{
		FByteProperty* ByteProperty = CastFieldChecked<FByteProperty>(State.ValueProperty);

		if (ByteProperty->IsEnum())
		{
			WritePropertyValue(State, ByteProperty->Enum->GetNameStringByValue(ByteProperty->GetPropertyValue_InContainer(State.ValueData, ArrayIndex)));
		}
		else
		{
			WritePropertyValue(State, ByteProperty->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
		}
	}

	// floating point numbers
	else if (State.FieldType == FDoubleProperty::StaticClass())
	{
		WritePropertyValue(State, CastFieldChecked<FDoubleProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FFloatProperty::StaticClass())
	{
		WritePropertyValue(State, CastFieldChecked<FFloatProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}

	// signed integers
	else if (State.FieldType == FIntProperty::StaticClass())
	{
		WritePropertyValue(State, CastFieldChecked<FIntProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FInt8Property::StaticClass())
	{
		WritePropertyValue(State, CastFieldChecked<FInt8Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FInt16Property::StaticClass())
	{
		WritePropertyValue(State, CastFieldChecked<FInt16Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FInt64Property::StaticClass())
	{
		WritePropertyValue(State, CastFieldChecked<FInt64Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}

	// unsigned integers
	else if (State.FieldType == FUInt16Property::StaticClass())
	{
		WritePropertyValue(State, CastFieldChecked<FUInt16Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FUInt32Property::StaticClass())
	{
		WritePropertyValue(State, CastFieldChecked<FUInt32Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FUInt64Property::StaticClass())
	{
		WritePropertyValue(State, CastFieldChecked<FUInt64Property>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}

	// names, strings & text
	else if (State.FieldType == FNameProperty::StaticClass())
	{
		WritePropertyValue(State, CastFieldChecked<FNameProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex).ToString());
	}
	else if (State.FieldType == FStrProperty::StaticClass())
	{
		WritePropertyValue(State, CastFieldChecked<FStrProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex));
	}
	else if (State.FieldType == FTextProperty::StaticClass())
	{
		const FText& TextValue = CastFieldChecked<FTextProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex);
		if (EnumHasAnyFlags(Flags, EStructSerializerBackendFlags::WriteTextAsComplexString))
		{
			FString TextValueString;
			FTextStringHelper::WriteToBuffer(TextValueString, TextValue);
			WritePropertyValue(State, TextValueString);
		}
		else
		{
			WritePropertyValue(State, TextValue.ToString());
		}
	}

	// classes & objects
	else if (State.FieldType == FSoftClassProperty::StaticClass())
	{
		FSoftObjectPtr const& Value = CastFieldChecked<FSoftClassProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex);
		WritePropertyValue(State, Value.IsValid() ? Value->GetPathName() : FString());
	}
	else if (State.FieldType == FWeakObjectProperty::StaticClass())
	{
		FWeakObjectPtr const& Value = CastFieldChecked<FWeakObjectProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex);
		WritePropertyValue(State, Value.IsValid() ? Value.Get()->GetPathName() : FString());
	}
	else if (State.FieldType == FSoftObjectProperty::StaticClass())
	{
		FSoftObjectPtr const& Value = CastFieldChecked<FSoftObjectProperty>(State.ValueProperty)->GetPropertyValue_InContainer(State.ValueData, ArrayIndex);
		WritePropertyValue(State, Value.ToString());
	}
	else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(State.ValueProperty))
	{
		// @TODO: Could this be expanded to include everything derived from FObjectPropertyBase?
		// Generic handling for a property type derived from FObjectProperty that is obtainable as a pointer and will be stored using its path.
		// This must come after all the more specialized handlers for object property types.
		UObject* const Value = ObjectProperty->GetObjectPropertyValue_InContainer(State.ValueData, ArrayIndex);
		WritePropertyValue(State, Value ? Value->GetPathName() : FString());
	}

	// unsupported property type
	else
	{
		UE_LOGF(LogSerialization, Verbose, "FJsonStructSerializerBackend: Property %ls cannot be serialized, because its type (%ls) is not supported", *State.ValueProperty->GetFName().ToString(), *State.ValueType->GetFName().ToString());
	}
}

bool FJsonStructSerializerBackend::WritePODArray(const FStructSerializerState& State)
{
	if (State.ElementIndex != INDEX_NONE)
	{
		return false;
	}

	// This code serializes TArray<uint8> in the same way FStructSerializer would do. However, we iterate over
	// elements directly while FStructSerializer allocates one FStructSerializerState (64 bytes) for each
	// byte item. As a result, one 1 MB of binary data would require more than 64 MB of temporary state data.
	FArrayProperty* ArrayProperty = CastField<FArrayProperty>(State.ValueProperty);
	if (ArrayProperty && (CastField<FByteProperty>(ArrayProperty->Inner) || CastField<FInt8Property>(ArrayProperty->Inner)))
	{
		FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(State.ValueData));

		FStructSerializerState InnerState(nullptr, ArrayProperty->Inner, EStructSerializerStateFlags::None);

		for (int32 Index = 0; Index < ArrayHelper.Num(); ++Index)
		{
			if (ArrayHelper.IsValidIndex(Index))
			{
				InnerState.ValueData = ArrayHelper.GetRawPtr(Index);

				WriteProperty(InnerState);
			}
		}

		// We need to close the array ourselves because FStructSerializer doesn't do this after we've declared we've completed
		// serialization by returning true.
		EndArray(State);

		return true;
	}

	return false;
}
