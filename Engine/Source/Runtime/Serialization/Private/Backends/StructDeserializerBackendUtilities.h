// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Class.h"
#include "UObject/StrProperty.h"
#include "UObject/UnrealType.h"
#include "UObject/EnumProperty.h"
#include "UObject/TextProperty.h"


struct StructDeserializerBackendUtilities
{
	/**
	* Clears the value of the given property.
	*
	* @param Property The property to clear.
	* @param Outer The property that contains the property to be cleared, if any.
	* @param Data A pointer to the memory holding the property's data.
	* @param ArrayIndex The index of the element to clear (if the property is an array).
	* @return true on success, false otherwise.
	* @see SetPropertyValue
	*/
	static bool ClearPropertyValue(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex)
	{
		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Outer);

		if (ArrayProperty != nullptr)
		{
			if (ArrayProperty->Inner != Property)
			{
				return false;
			}

			FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->template ContainerPtrToValuePtr<void>(Data));
			ArrayIndex = ArrayHelper.AddValue();
		}

		Property->ClearValue_InContainer(Data, ArrayIndex);

		return true;
	}


	/**
	* Gets a pointer to object of the given property.
	*
	* @param Property The property to get.
	* @param Outer The property that contains the property to be get, if any.
	* @param Data A pointer to the memory holding the property's data.
	* @param ArrayIndex The index of the element to set (if the property is an array).
	* @return A pointer to the object represented by the property, null otherwise..
	* @see ClearPropertyValue
	*/
	static void* GetPropertyValuePtr(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex)
	{
		check(Property);

		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Outer))
		{
			if (ArrayProperty->Inner != Property)
			{
				return nullptr;
			}

			FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->template ContainerPtrToValuePtr<void>(Data));
			int32 Index = ArrayHelper.AddValue();

			return ArrayHelper.GetRawPtr(Index);
		}

		if (ArrayIndex >= Property->ArrayDim)
		{
			return nullptr;
		}

		return Property->template ContainerPtrToValuePtr<void>(Data, ArrayIndex);
	}

	/**
	* Sets the value of the given property.
	*
	* @param Property The property to set.
	* @param Outer The property that contains the property to be set, if any.
	* @param Data A pointer to the memory holding the property's data.
	* @param ArrayIndex The index of the element to set (if the property is an array).
	* @return true on success, false otherwise.
	* @see ClearPropertyValue
	*/
	template<typename PropertyType, typename ValueType>
	static bool SetPropertyValue(PropertyType* Property, FProperty* Outer, void* Data, int32 ArrayIndex, const ValueType& Value)
	{
		if (void* Ptr = GetPropertyValuePtr(Property, Outer, Data, ArrayIndex))
		{
			Property->SetPropertyValue(Ptr, Value);
			return true;
		}

		return false;
	}
};
