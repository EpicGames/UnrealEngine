// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructDeserializer.h"
#include "UObject/UnrealType.h"
#include "IStructDeserializerBackend.h"
#include "UObject/PropertyOptional.h"
#include "UObject/PropertyPortFlags.h"


/* Internal helpers
 *****************************************************************************/

namespace StructDeserializer
{
	/**
	 * Structure for the read state stack.
	 */
	struct FReadState
	{
		/** Holds the property's current array index. */
		int32 ArrayIndex = 0;

		/** Holds a pointer to the property's data. */
		void* Data = nullptr;

		/** Holds the property's meta data. */
		FProperty* Property = nullptr;

		/** Holds a pointer to the UStruct describing the data. */
		UStruct* TypeInfo = nullptr;
	};


	/**
	 * Finds the class for the given stack state.
	 *
	 * @param State The stack state to find the class for.
	 * @return The class, if found.
	 */
	UStruct* FindClass( const FReadState& State )
	{
		UStruct* Class = nullptr;

		if (State.Property != nullptr)
		{
			FProperty* ParentProperty = State.Property;

			if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ParentProperty))
			{
				ParentProperty = ArrayProperty->Inner;
			}

			FStructProperty* StructProperty = CastField<FStructProperty>(ParentProperty);
			FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(ParentProperty);

			if (StructProperty != nullptr)
			{
				Class = StructProperty->Struct;
			}
			else if (ObjectProperty != nullptr)
			{
				Class = ObjectProperty->PropertyClass;
			}
		}
		else
		{
			UObject* RootObject = static_cast<UObject*>(State.Data);
			Class = RootObject->GetClass();
		}

		return Class;
	}

	/** Finds an element in a Map/Set Container at the given logical index, adding enough entries for that logical index to be valid */
	template<typename ScriptContainerHelperType>
	int32 ExpandForIndex(ScriptContainerHelperType& ScriptContainerHelper, int32 LogicalIndex)
	{
		int32 InternalIndex = ScriptContainerHelper.FindInternalIndex(LogicalIndex);
		if (InternalIndex != INDEX_NONE)
		{
			return InternalIndex;
		}

		// Container does not have any element at the logical array index, add enough items so that this logical index is valid
		const int32 ItemsToAddCount = LogicalIndex - ScriptContainerHelper.Num() + 1;

		// Maps/Set containers don't have a ExpandForIndex / Add(Count). Add default values manually
		for (int32 ItemsAddedCount = 0; ItemsAddedCount < ItemsToAddCount; ++ItemsAddedCount)
		{
			ScriptContainerHelper.AddDefaultValue_Invalid_NeedsRehash();
		}

		InternalIndex = ScriptContainerHelper.FindInternalIndex(LogicalIndex);
		check(InternalIndex != INDEX_NONE);
		return InternalIndex;
	}
}


/* FStructDeserializer static interface
 *****************************************************************************/

bool FStructDeserializer::Deserialize( void* OutStruct, UStruct& TypeInfo, IStructDeserializerBackend& Backend, const FStructDeserializerPolicies& Policies )
{
	using namespace StructDeserializer;

	check(OutStruct != nullptr);

	// initialize deserialization
	FReadState CurrentState;
	{
		CurrentState.ArrayIndex = 0;
		CurrentState.Data = OutStruct;
		CurrentState.Property = nullptr;
		CurrentState.TypeInfo = &TypeInfo;
	}

	TArray<FReadState> StateStack;
	EStructDeserializerBackendTokens Token;

	// process state stack
	while (Backend.GetNextToken(Token))
	{
		FString PropertyName = Backend.GetCurrentPropertyName();

		switch (Token)
		{
		case EStructDeserializerBackendTokens::ArrayEnd:
			{
				// rehash the set now that we are done with it
				if (FSetProperty* SetProperty = CastField<FSetProperty>(CurrentState.Property))
				{
					FScriptSetHelper SetHelper(SetProperty, CurrentState.Data);
					SetHelper.Rehash();
				}

				if (StateStack.Num() == 0)
				{
					UE_LOGF(LogSerialization, Verbose, "Malformed input: Found ArrayEnd without matching ArrayStart");

					return false;
				}

				CurrentState = StateStack.Pop(EAllowShrinking::No);
			}
			break;

		case EStructDeserializerBackendTokens::ArrayStart:
			{
				FReadState NewState;

				NewState.Property = FindFProperty<FProperty>(CurrentState.TypeInfo, *PropertyName);

				if (NewState.Property != nullptr)
				{
					if (Policies.PropertyFilter && !Policies.PropertyFilter(NewState.Property, CurrentState.Property))
					{
						Backend.SkipArray();
						continue;
					}

					// handle set property
					if (FSetProperty* SetProperty = CastField<FSetProperty>(NewState.Property))
					{
						NewState.Data = SetProperty->ContainerPtrToValuePtr<void>(CurrentState.Data, CurrentState.ArrayIndex);
						FScriptSetHelper SetHelper(SetProperty, NewState.Data);
						SetHelper.EmptyElements();
					}
					// handle array property
					else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(NewState.Property))
					{
						// fast path for byte array 
						if (Backend.ReadPODArray(ArrayProperty, CurrentState.Data))
						{
							// read the entire array, move to the next property
							continue;
						}
						// failed to read as a pod array, read as regular array iterating on each property
						else
						{
							NewState.Data = CurrentState.Data;
						}
					}
					// handle static array	
					else
					{
						NewState.Data = CurrentState.Data;
					}

					NewState.TypeInfo = FindClass(NewState);
					StateStack.Push(CurrentState);
					CurrentState = NewState;
				}
				else
				{
					// error: array property not found
					if (Policies.MissingFields != EStructDeserializerErrorPolicies::Ignore)
					{
						UE_LOGF(LogSerialization, Verbose, "The array property '%ls' does not exist", *PropertyName);
					}

					if (Policies.MissingFields == EStructDeserializerErrorPolicies::Error)
					{
						return false;
					}

					Backend.SkipArray();
				}
			}
			break;

		case EStructDeserializerBackendTokens::Error:
			{
				return false;
			}

		case EStructDeserializerBackendTokens::Property:
			{
				// Set are serialized as array, so no property name will be set for each entry
				if (PropertyName.IsEmpty() && (CurrentState.Property != nullptr) && (CurrentState.Property->GetClass() == FSetProperty::StaticClass()))
				{
					// handle set element
					FSetProperty* SetProperty = CastField<FSetProperty>(CurrentState.Property);
					FScriptSetHelper SetHelper(SetProperty, CurrentState.Data);
					FProperty* Property = SetProperty->ElementProp;

					const int32 ElementIndex = SetHelper.AddDefaultValue_Invalid_NeedsRehash();
					uint8* ElementPtr = SetHelper.GetElementPtr(ElementIndex);

					if (!Backend.ReadProperty(Property, CurrentState.Property, ElementPtr, CurrentState.ArrayIndex))
					{
						UE_LOGF(LogSerialization, Verbose, "An item in Set '%ls' could not be read (%ls)", *PropertyName, *Backend.GetDebugString());
					}
				}
				// Otherwise we are dealing with dynamic or static array
				else if (PropertyName.IsEmpty())
				{
					// handle array element
					FArrayProperty* ArrayProperty = CastField<FArrayProperty>(CurrentState.Property);
					FProperty* Property = nullptr;

					if (ArrayProperty != nullptr)
					{
						// dynamic array element
						Property = ArrayProperty->Inner;
					}
					else
					{
						// static array element
						Property = CurrentState.Property;
					}

					if (Property == nullptr)
					{
						// error: no meta data for array element
						if (Policies.MissingFields != EStructDeserializerErrorPolicies::Ignore)
						{
							UE_LOGF(LogSerialization, Verbose, "Failed to serialize array element %i", CurrentState.ArrayIndex);
						}

						return false;
					}
					else if (!Backend.ReadProperty(Property, CurrentState.Property, CurrentState.Data, CurrentState.ArrayIndex))
					{
						UE_LOGF(LogSerialization, Verbose, "The array element '%ls[%i]' could not be read (%ls)", *PropertyName, CurrentState.ArrayIndex, *Backend.GetDebugString());
					}

					++CurrentState.ArrayIndex;
				}
				else if ((CurrentState.Property != nullptr) && (CurrentState.Property->GetClass() == FMapProperty::StaticClass()))
				{
					// handle map element
					FMapProperty* MapProperty = CastField<FMapProperty>(CurrentState.Property);
					FScriptMapHelper MapHelper(MapProperty, CurrentState.Data);
					FProperty* Property = MapProperty->ValueProp;

					int32 PairIndex = MapHelper.AddDefaultValue_Invalid_NeedsRehash();
					uint8* PairPtr = MapHelper.GetPairPtr(PairIndex);

					MapProperty->KeyProp->ImportText_Direct(*PropertyName, PairPtr, nullptr, PPF_None);

					if (!Backend.ReadProperty(Property, CurrentState.Property, PairPtr, CurrentState.ArrayIndex))
					{
						UE_LOGF(LogSerialization, Verbose, "An item in map '%ls' could not be read (%ls)", *PropertyName, *Backend.GetDebugString());
					}
				}
				else if ((CurrentState.Property != nullptr) && (CurrentState.Property->GetClass() == FOptionalProperty::StaticClass()))
				{
					FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(CurrentState.Property);
					FProperty* Property = OptionalProperty->GetValueProperty();
					void* ValueData = OptionalProperty->MarkSetAndGetInitializedValuePointerToReplace(CurrentState.Data);

					if (!Backend.ReadProperty(Property, CurrentState.Property, ValueData, CurrentState.ArrayIndex))
					{
						UE_LOGF(LogSerialization, Verbose, "An item in optional '%ls' could not be read (%ls)", *PropertyName, *Backend.GetDebugString());
					}
				}
				else
				{
					// handle scalar property
					FProperty* Property = FindFProperty<FProperty>(CurrentState.TypeInfo, *PropertyName);

					if (Property != nullptr)
					{
						if (Policies.PropertyFilter && !Policies.PropertyFilter(Property, CurrentState.Property))
						{
							continue;
						}

						if (!Backend.ReadProperty(Property, CurrentState.Property, CurrentState.Data, CurrentState.ArrayIndex))
						{
							UE_LOGF(LogSerialization, Verbose, "The property '%ls' could not be read (%ls)", *PropertyName, *Backend.GetDebugString());
						}
					}
					else
					{
						// error: scalar property not found
						if (Policies.MissingFields != EStructDeserializerErrorPolicies::Ignore)
						{
							UE_LOGF(LogSerialization, Verbose, "The property '%ls' does not exist", *PropertyName);
						}

						if (Policies.MissingFields == EStructDeserializerErrorPolicies::Error)
						{
							return false;
						}
					}
				}
			}
			break;

		case EStructDeserializerBackendTokens::StructureEnd:
			{
				// rehash if value was a map
				FMapProperty* MapProperty = CastField<FMapProperty>(CurrentState.Property);
				if (MapProperty != nullptr)
				{			
					FScriptMapHelper MapHelper(MapProperty, CurrentState.Data);
					MapHelper.Rehash();
				}

				// ending of root structure
				if (StateStack.Num() == 0)
				{
					return true;
				}

				CurrentState = StateStack.Pop(EAllowShrinking::No);
			}
			break;

		case EStructDeserializerBackendTokens::StructureStart:
			{
				FReadState NewState;

				if (PropertyName.IsEmpty())
				{
					// skip root structure
					if (CurrentState.Property == nullptr)
					{
						check(StateStack.Num() == 0);
						continue;
					}

					// handle struct element inside set
					if (FSetProperty* SetProperty = CastField<FSetProperty>(CurrentState.Property))
					{
						FScriptSetHelper SetHelper(SetProperty, CurrentState.Data);
						const int32 ElementIndex = SetHelper.AddDefaultValue_Invalid_NeedsRehash();
						uint8* ElementPtr = SetHelper.GetElementPtr(ElementIndex);

						NewState.Data = ElementPtr;
						NewState.Property = SetProperty->ElementProp;
					}
					// handle struct element inside array
					else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(CurrentState.Property))
					{
						FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(CurrentState.Data));
						const int32 ArrayIndex = ArrayHelper.AddValue();

						NewState.Property = ArrayProperty->Inner;
						NewState.Data = ArrayHelper.GetRawPtr(ArrayIndex);
					}
					else
					{
						UE_LOGF(LogSerialization, Verbose, "Found unnamed value outside of array or set.");
						return false;
					}
				}
				// handle map or struct element inside map
				else if ((CurrentState.Property != nullptr) && (CurrentState.Property->GetClass() == FMapProperty::StaticClass()))
				{
					FMapProperty* MapProperty = CastField<FMapProperty>(CurrentState.Property);
					FScriptMapHelper MapHelper(MapProperty, CurrentState.Data);
					int32 PairIndex = MapHelper.AddDefaultValue_Invalid_NeedsRehash();
					uint8* PairPtr = MapHelper.GetPairPtr(PairIndex);
					
					NewState.Data = PairPtr + MapHelper.MapLayout.ValueOffset;
					NewState.Property = MapProperty->ValueProp;

					MapProperty->KeyProp->ImportText_Direct(*PropertyName, PairPtr, nullptr, PPF_None);
				}
				// handle map or struct element inside optional
				else if ((CurrentState.Property != nullptr) && (CurrentState.Property->GetClass() == FOptionalProperty::StaticClass()))
				{
					FOptionalProperty* OptionalProperty = CastFieldChecked<FOptionalProperty>(CurrentState.Property);
					NewState.Property = OptionalProperty->GetValueProperty();
					NewState.Data = OptionalProperty->MarkSetAndGetInitializedValuePointerToReplace(CurrentState.Data);
				}
				else
				{
					NewState.Property = FindFProperty<FProperty>(CurrentState.TypeInfo, *PropertyName);

					// unrecognized property
					if (NewState.Property == nullptr)
					{
						// error: map or struct property not found
						if (Policies.MissingFields != EStructDeserializerErrorPolicies::Ignore)
						{
							UE_LOGF(LogSerialization, Verbose, "Map, Set, or struct property '%ls' not found", *PropertyName);
						}

						if (Policies.MissingFields == EStructDeserializerErrorPolicies::Error)
						{
							return false;
						}
					}
					// handle map property start
					else if (FMapProperty* MapProperty = CastField<FMapProperty>(NewState.Property))
					{
						NewState.Data = MapProperty->ContainerPtrToValuePtr<void>(CurrentState.Data, CurrentState.ArrayIndex);
						FScriptMapHelper MapHelper(MapProperty, NewState.Data);
						MapHelper.EmptyValues();
					}
					// handle optional property start
					else if (FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(NewState.Property))
					{
						NewState.Data = OptionalProperty->ContainerPtrToValuePtr<void>(CurrentState.Data, CurrentState.ArrayIndex);
						OptionalProperty->MarkUnset(NewState.Data);
					}
					// handle struct property
					else
					{
						NewState.Data = NewState.Property->ContainerPtrToValuePtr<void>(CurrentState.Data);
					}
				}

				if (NewState.Property != nullptr)
				{
					// skip struct property if property filter is set and rejects it
					if (Policies.PropertyFilter && !Policies.PropertyFilter(NewState.Property, CurrentState.Property))
					{
						Backend.SkipStructure();
						continue;
					}

					NewState.ArrayIndex = 0;
					NewState.TypeInfo = FindClass(NewState);

					StateStack.Push(CurrentState);
					CurrentState = NewState;
				}
				else
				{
					// error: structured property not found
					Backend.SkipStructure();

					if (Policies.MissingFields != EStructDeserializerErrorPolicies::Ignore)
					{
						UE_LOGF(LogSerialization, Verbose, "Structured property '%ls' not found", *PropertyName);
					}

					if (Policies.MissingFields == EStructDeserializerErrorPolicies::Error)
					{
						return false;
					}
				}
			}

		default:

			continue;
		}
	}

	// root structure not completed
	return false;
}
bool FStructDeserializer::DeserializeElement(void* OutAddress, UStruct& OwnerInfo, int32 InElementIndex, IStructDeserializerBackend& Backend, const FStructDeserializerPolicies& Policies)
{
	using namespace StructDeserializer;

	check(OutAddress != nullptr);

	// initialize deserialization
	FReadState CurrentState;
	{
		CurrentState.ArrayIndex = InElementIndex == INDEX_NONE ? 0 : InElementIndex;
		CurrentState.Data = OutAddress;
		CurrentState.TypeInfo = &OwnerInfo;
	}

	TArray<FReadState> StateStack;
	EStructDeserializerBackendTokens Token;

	// process state stack
	while (Backend.GetNextToken(Token))
	{
		const FString PropertyName = Backend.GetCurrentPropertyName();

		switch (Token)
		{
		case EStructDeserializerBackendTokens::ArrayEnd:
		{
			// rehash the set/maps -> we're closing them
			check(CurrentState.Property);
			if (FSetProperty* SetProperty = CastField<FSetProperty>(CurrentState.Property))
			{
				FScriptSetHelper SetHelper(SetProperty, CurrentState.Data);
				SetHelper.Rehash();
			}
			else if (FMapProperty* MapProperty = CastField<FMapProperty>(CurrentState.Property))
			{
				FScriptMapHelper MapHelper(MapProperty, CurrentState.Data);
				MapHelper.Rehash();
			}
			else if (CurrentState.Property->ArrayDim > 1 && CurrentState.ArrayIndex < CurrentState.Property->ArrayDim)
			{
				// error: array entry not found in static array
				if (Policies.MissingFields != EStructDeserializerErrorPolicies::Ignore)
				{
					UE_LOGF(LogSerialization, Verbose, "The static array '%ls' of size %d only had %d entries", *CurrentState.Property->GetFName().ToString(), CurrentState.Property->ArrayDim, CurrentState.ArrayIndex);
				}

				if (Policies.MissingFields == EStructDeserializerErrorPolicies::Error)
				{
					return false;
				}
			}

			if (StateStack.Num() == 0)
			{
				UE_LOGF(LogSerialization, Verbose, "Malformed input: Found ArrayEnd without matching ArrayStart");

				return false;
			}

			CurrentState = StateStack.Pop(EAllowShrinking::No);
		}
		break;

		case EStructDeserializerBackendTokens::ArrayStart:
		{
			FReadState NewState;
			NewState.Property = FindFProperty<FProperty>(CurrentState.TypeInfo, *PropertyName);

			if (NewState.Property != nullptr)
			{
				if (Policies.PropertyFilter && !Policies.PropertyFilter(NewState.Property, CurrentState.Property))
				{
					Backend.SkipArray();
					continue;
				}

				if (FSetProperty* SetProperty = CastField<FSetProperty>(NewState.Property))
				{
					NewState.Data = SetProperty->ContainerPtrToValuePtr<void>(CurrentState.Data);
					NewState.ArrayIndex = 0;
				}
				else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(NewState.Property))
				{
					NewState.Data = CurrentState.Data;
					NewState.ArrayIndex = 0;
				}
				else if (FMapProperty* MapProperty = CastField<FMapProperty>(NewState.Property))
				{
					NewState.Data = MapProperty->ContainerPtrToValuePtr<void>(CurrentState.Data);
					NewState.ArrayIndex = 0;
				}
				// static array property
				else if (NewState.Property != nullptr)
				{
					NewState.Data = CurrentState.Data;
					NewState.ArrayIndex = 0;
				}

				NewState.TypeInfo = FindClass(NewState);
				StateStack.Push(CurrentState);
				CurrentState = NewState;
			}
			else
			{
				// error: array property not found
				if (Policies.MissingFields != EStructDeserializerErrorPolicies::Ignore)
				{
					UE_LOGF(LogSerialization, Verbose, "The property '%ls' does not exist", *PropertyName);
				}

				if (Policies.MissingFields == EStructDeserializerErrorPolicies::Error)
				{
					return false;
				}

				Backend.SkipArray();
			}
		}
		break;

		case EStructDeserializerBackendTokens::Error:
		{
			return false;
		}

		case EStructDeserializerBackendTokens::Property:
		{
			// Set are serialized as array, so no property name will be set for each entry
			if (PropertyName.IsEmpty() && (CurrentState.Property != nullptr) && (CurrentState.Property->GetClass() == FSetProperty::StaticClass()))
			{
				// handle set element
				FSetProperty* SetProperty = CastField<FSetProperty>(CurrentState.Property);
				FScriptSetHelper SetHelper(SetProperty, CurrentState.Data);

				const int32 InternalIndex = ExpandForIndex(SetHelper, CurrentState.ArrayIndex);

				uint8* ElementPtr = SetHelper.GetElementPtr(InternalIndex);
				FProperty* Property = SetProperty->ElementProp;
				constexpr int32 ReadIndex = 0; //Pointer is offset so reading index is 0
				if (!Backend.ReadProperty(Property, CurrentState.Property, ElementPtr, ReadIndex))
				{
					UE_LOGF(LogSerialization, Verbose, "An item in Set '%ls' could not be read (%ls)", *PropertyName, *Backend.GetDebugString());
				}
				++CurrentState.ArrayIndex;
			}
			// Maps can be serialized as array, so no property name will be set for each entry. Each entry will be taken in order
			if (PropertyName.IsEmpty() && (CurrentState.Property != nullptr) && (CurrentState.Property->GetClass() == FMapProperty::StaticClass()))
			{
				// handle map element
				FMapProperty* MapProperty = CastField<FMapProperty>(CurrentState.Property);
				FScriptMapHelper MapHelper(MapProperty, CurrentState.Data);

				//When written as array, maps won't include the key, only values
				const int32 InternalIndex = ExpandForIndex(MapHelper, CurrentState.ArrayIndex);

				uint8* PairPtr = MapHelper.GetPairPtr(InternalIndex);
				FProperty* Property = MapProperty->ValueProp;
				constexpr int32 ReadIndex = 0; //Pointer is offset so reading index is 0
				if (!Backend.ReadProperty(Property, CurrentState.Property, PairPtr, ReadIndex))
				{
					UE_LOGF(LogSerialization, Verbose, "An item in Set '%ls' could not be read (%ls)", *PropertyName, *Backend.GetDebugString());
				}
				
				++CurrentState.ArrayIndex;
			}
			else if ((CurrentState.Property != nullptr) && (CurrentState.Property->GetClass() == FOptionalProperty::StaticClass()))
			{
				FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(CurrentState.Property);
				FProperty* Property = OptionalProperty->GetValueProperty();
				void* ValueData = OptionalProperty->MarkSetAndGetInitializedValuePointerToReplace(CurrentState.Data);

				if (!Backend.ReadProperty(Property, CurrentState.Property, ValueData, CurrentState.ArrayIndex))
				{
					UE_LOGF(LogSerialization, Verbose, "An item in optional '%ls' could not be read (%ls)", *PropertyName, *Backend.GetDebugString());
				}
			}
			// Otherwise we are dealing with dynamic or static array
			else if (PropertyName.IsEmpty())
			{
				// When reading the property, the deserialize behavior is to add element. We bypass that with the property
				FArrayProperty* ArrayProperty = CastField<FArrayProperty>(CurrentState.Property);
				FProperty* Property = nullptr;
				void* DataAddress = CurrentState.Data;
				int32 CurrentArrayIndex = CurrentState.ArrayIndex;

				if (ArrayProperty != nullptr)
				{
					// dynamic array element
					FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(CurrentState.Data));
					ArrayHelper.ExpandForIndex(CurrentState.ArrayIndex);

					Property = ArrayProperty->Inner;
					DataAddress = ArrayHelper.GetRawPtr(CurrentState.ArrayIndex);

					//Arraydim will be 1 for inner TArray properties. Offset the read data and keep index at 0
					CurrentArrayIndex = 0; 
				}
				else
				{
					checkSlow(CurrentState.Property);
					// static array element
					if (CurrentState.ArrayIndex >= 0 && CurrentState.ArrayIndex < CurrentState.Property->ArrayDim)
					{
						Property = CurrentState.Property;
					}
					else
					{
						//Too many entries in static Array
						UE_LOGF(LogSerialization, Verbose, "Static array %ls has dimension of %d and trying to read element %d", *CurrentState.Property->GetFName().ToString(), CurrentState.Property->ArrayDim, CurrentState.ArrayIndex);
						continue;
					}
				}

				if (Property == nullptr)
				{
					// error: no meta data for array element
					if (Policies.MissingFields != EStructDeserializerErrorPolicies::Ignore)
					{
						UE_LOGF(LogSerialization, Verbose, "Failed to serialize array element %i", CurrentState.ArrayIndex);
					}

					return false;
				}
				else if (!Backend.ReadProperty(Property, nullptr, DataAddress, CurrentArrayIndex))
				{
					UE_LOGF(LogSerialization, Verbose, "The array element '%ls[%i]' could not be read (%ls)", *PropertyName, CurrentState.ArrayIndex, *Backend.GetDebugString());
				}

				++CurrentState.ArrayIndex;
			}
			else
			{
				// handle scalar property
				
				FProperty* Property = FindFProperty<FProperty>(CurrentState.TypeInfo, *PropertyName);

				if (Property != nullptr)
				{
					if (Policies.PropertyFilter && !Policies.PropertyFilter(Property, CurrentState.Property))
					{
						continue;
					}

					//Direct set element
					if (FSetProperty* SetProperty = CastField<FSetProperty>(Property))
					{
						FScriptSetHelper SetHelper(SetProperty, SetProperty->ContainerPtrToValuePtr<void>(CurrentState.Data));

						const int32 InternalIndex = ExpandForIndex(SetHelper, CurrentState.ArrayIndex);

						Property = SetProperty->ElementProp;

						//Offset the pointer directly and give index 0 to be read so no offsetting is done during deserialization
						CurrentState.Data = SetHelper.GetElementPtr(InternalIndex);
						CurrentState.ArrayIndex = 0;

						if (!Backend.ReadProperty(Property, nullptr, CurrentState.Data, CurrentState.ArrayIndex))
						{
							UE_LOGF(LogSerialization, Verbose, "The property '%ls' could not be read (%ls)", *PropertyName, *Backend.GetDebugString());
						}

						//An element of a set was written so rehash it
						SetHelper.Rehash();
						continue;
					}
					else if (FMapProperty* MapProperty = CastField<FMapProperty>(Property))
					{
						FScriptMapHelper MapHelper(MapProperty, MapProperty->ContainerPtrToValuePtr<void>(CurrentState.Data));

						const int32 InternalIndex = ExpandForIndex(MapHelper, CurrentState.ArrayIndex);

						Property = MapProperty->ValueProp;

						//Offset the pointer directly and give index 0 to be read so no offsetting is done during deserialization
						CurrentState.Data = MapHelper.GetPairPtr(InternalIndex);
						CurrentState.ArrayIndex = 0;

						if (!Backend.ReadProperty(Property, nullptr, CurrentState.Data, CurrentState.ArrayIndex))
						{
							UE_LOGF(LogSerialization, Verbose, "The property '%ls' could not be read (%ls)", *PropertyName, *Backend.GetDebugString());
						}

						//An element of a map was written so rehash it
						MapHelper.Rehash();
						continue;
					}
					else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
					{
						FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(CurrentState.Data));
						ArrayHelper.ExpandForIndex(CurrentState.ArrayIndex);

						Property = ArrayProperty->Inner;

						//Offset the pointer directly and give index 0 to be read so no offsetting is done during deserialization
						CurrentState.Data = ArrayHelper.GetRawPtr(CurrentState.ArrayIndex);
						CurrentState.ArrayIndex = 0;
					}
					else if (FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(Property))
					{
						void* Data = OptionalProperty->ContainerPtrToValuePtr<void>(CurrentState.Data);

						if (void* ValueData = OptionalProperty->GetValuePointerForReadOrReplaceIfSet(Data))
						{
							Property = OptionalProperty->GetValueProperty();

							//Offset the pointer directly and give index 0 to be read so no offsetting is done during deserialization
							CurrentState.Data = ValueData;
							CurrentState.ArrayIndex = 0;
						}
						else
						{
							// Not set
							UE_LOGF(LogSerialization, Verbose, "TOptional %ls is not set and is trying to be read", *OptionalProperty->GetFName().ToString());
							Backend.SkipStructure();
							continue;
						}
					}

					if (!Backend.ReadProperty(Property, nullptr, CurrentState.Data, CurrentState.ArrayIndex))
					{
						UE_LOGF(LogSerialization, Verbose, "The property '%ls' could not be read (%ls)", *PropertyName, *Backend.GetDebugString());
					}
				}
				else
				{
					// error: scalar property not found
					if (Policies.MissingFields != EStructDeserializerErrorPolicies::Ignore)
					{
						UE_LOGF(LogSerialization, Verbose, "The property '%ls' does not exist", *PropertyName);
					}

					if (Policies.MissingFields == EStructDeserializerErrorPolicies::Error)
					{
						return false;
					}
				}
			}
		}
		break;

		case EStructDeserializerBackendTokens::StructureEnd:
		{
			// rehash if value was a map, set
			if(FMapProperty* MapProperty = CastField<FMapProperty>(CurrentState.Property))
			{
				FScriptMapHelper MapHelper(MapProperty, CurrentState.Data);
				MapHelper.Rehash();
			}
			else if (FSetProperty* SetProperty = CastField<FSetProperty>(CurrentState.Property))
			{
				FScriptSetHelper SetHelper(SetProperty, CurrentState.Data);
				SetHelper.Rehash();
			}

			// ending of root structure
			if (StateStack.Num() == 0)
			{
				return true;
			}

			CurrentState = StateStack.Pop(EAllowShrinking::No);
		}
		break;

		case EStructDeserializerBackendTokens::StructureStart:
		{
			FReadState NewState;

			if (PropertyName.IsEmpty())
			{
				// skip root structure
				if (CurrentState.Property == nullptr)
				{
					check(StateStack.Num() == 0);
					continue;
				}

				// handle struct element inside set
				if (FSetProperty* SetProperty = CastField<FSetProperty>(CurrentState.Property))
				{
					FScriptSetHelper SetHelper(SetProperty, CurrentState.Data);

					const int32 InternalIndex = ExpandForIndex(SetHelper, CurrentState.ArrayIndex);

					NewState.Property = SetProperty->ElementProp;
					NewState.Data = SetHelper.GetElementPtr(InternalIndex);
					NewState.ArrayIndex = 0;
					++CurrentState.ArrayIndex;
				}
				else if (FMapProperty* MapProperty = CastField<FMapProperty>(CurrentState.Property))
				{
					FScriptMapHelper MapHelper(MapProperty, CurrentState.Data);

					const int32 InternalIndex = ExpandForIndex(MapHelper, CurrentState.ArrayIndex);

					NewState.Property = MapProperty->ValueProp;
					NewState.Data = MapHelper.GetValuePtr(InternalIndex);
					NewState.ArrayIndex = 0;
					++CurrentState.ArrayIndex;
				}
				// handle struct element inside array
				else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(CurrentState.Property))
				{
					FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(CurrentState.Data));

					ArrayHelper.ExpandForIndex(CurrentState.ArrayIndex);

					NewState.Property = ArrayProperty->Inner;
					NewState.Data = ArrayHelper.GetRawPtr(CurrentState.ArrayIndex);
					NewState.ArrayIndex = 0;
					++CurrentState.ArrayIndex;
				}
				// handle struct element inside optional
				else if (FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(CurrentState.Property))
				{
					void* Data = OptionalProperty->ContainerPtrToValuePtr<void>(CurrentState.Data);

					NewState.Property = OptionalProperty->GetValueProperty();
					NewState.Data = OptionalProperty->MarkSetAndGetInitializedValuePointerToReplace(Data);
					NewState.ArrayIndex = 0;
				}
				else
				{
					//Property was found so we might be in a static array of struct
					if (CurrentState.ArrayIndex >= 0 && CurrentState.ArrayIndex < CurrentState.Property->ArrayDim)
					{
						NewState.Property = CurrentState.Property;
						NewState.Data = NewState.Property->ContainerPtrToValuePtr<void>(CurrentState.Data, CurrentState.ArrayIndex);
						NewState.ArrayIndex = 0;
						++CurrentState.ArrayIndex;
					}
					else
					{
						//Too many entries in static Array
						UE_LOGF(LogSerialization, Verbose, "Static array %ls has dimension of %d and trying to read element %d", *CurrentState.Property->GetFName().ToString(), CurrentState.Property->ArrayDim, CurrentState.ArrayIndex);
						Backend.SkipStructure();
						continue;
					}
				}
			}
			// handle map or struct element inside optional
			else if ((CurrentState.Property != nullptr) && (CurrentState.Property->GetClass() == FOptionalProperty::StaticClass()))
			{
				FOptionalProperty* OptionalProperty = CastFieldChecked<FOptionalProperty>(CurrentState.Property);
				NewState.Property = OptionalProperty->GetValueProperty();
				NewState.Data = OptionalProperty->MarkSetAndGetInitializedValuePointerToReplace(CurrentState.Data);
			}
			else
			{
				NewState.Property = FindFProperty<FProperty>(CurrentState.TypeInfo, *PropertyName);

				// unrecognized property
				if (NewState.Property == nullptr)
				{
					// error: map or struct property not found
					if (Policies.MissingFields != EStructDeserializerErrorPolicies::Ignore)
					{
						UE_LOGF(LogSerialization, Verbose, "Map, Set, or struct property '%ls' not found", *PropertyName);
					}

					if (Policies.MissingFields == EStructDeserializerErrorPolicies::Error)
					{
						return false;
					}
				}
				// handle map property start
				else if (FMapProperty* MapProperty = CastField<FMapProperty>(NewState.Property))
				{
					if (FStructProperty* SetStructProperty = CastField<FStructProperty>(MapProperty->ValueProp))
					{
						FScriptMapHelper MapHelper(MapProperty, MapProperty->ContainerPtrToValuePtr<void>(CurrentState.Data));

						const int32 InternalIndex = ExpandForIndex(MapHelper, CurrentState.ArrayIndex);

						// We're skipping a level directly so CurrentState becomes the outer (set) and NewState the inner (Element prop)
						CurrentState.Property = MapProperty;
						CurrentState.Data = MapProperty->ContainerPtrToValuePtr<void>(CurrentState.Data);

						NewState.Property = SetStructProperty;
						NewState.Data = MapHelper.GetValuePtr(InternalIndex);
						NewState.ArrayIndex = 0;
					}
				}
				//Handle Set property entry
				else if (FSetProperty* SetProperty = CastField<FSetProperty>(NewState.Property))
				{
					if (FStructProperty* SetStructProperty = CastField<FStructProperty>(SetProperty->ElementProp))
					{
						FScriptSetHelper SetHelper(SetProperty, SetProperty->ContainerPtrToValuePtr<void>(CurrentState.Data));

						const int32 InternalIndex = ExpandForIndex(SetHelper, CurrentState.ArrayIndex);

						//We're skipping a level directly so CurrentState becomes the outer (set) and NewState the inner (Element prop)
						CurrentState.Property = SetProperty;
						CurrentState.Data = SetProperty->ContainerPtrToValuePtr<void>(CurrentState.Data);

						NewState.Property = SetProperty->ElementProp;
						NewState.Data = SetHelper.GetElementPtr(InternalIndex);
						NewState.ArrayIndex = 0;
					}
				}
				//Handle array property entry
				else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(NewState.Property))
				{
					if (FStructProperty* ArrayStructProperty = CastField<FStructProperty>(ArrayProperty->Inner))
					{
						FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(CurrentState.Data));

						ArrayHelper.ExpandForIndex(CurrentState.ArrayIndex);

						//NewState will become the outer when going through the properties.
						//When reading from Array, we expect to read from one level. When its a struct, it's not.
						CurrentState.Property = ArrayProperty;
						CurrentState.Data = ArrayProperty->ContainerPtrToValuePtr<void>(CurrentState.Data);

						NewState.Property = ArrayProperty->Inner;
						NewState.Data = ArrayHelper.GetRawPtr(CurrentState.ArrayIndex);
						NewState.ArrayIndex = 0;
					}
				}
				// handle optional property entry
				else if (FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(NewState.Property))
				{
					NewState.Data = OptionalProperty->ContainerPtrToValuePtr<void>(CurrentState.Data, CurrentState.ArrayIndex);
					OptionalProperty->MarkUnset(NewState.Data);
				}
				// handle struct property
				else
				{
					if (CurrentState.ArrayIndex >= 0 && CurrentState.ArrayIndex < NewState.Property->ArrayDim)
					{
						NewState.Data = NewState.Property->ContainerPtrToValuePtr<void>(CurrentState.Data, CurrentState.ArrayIndex);
						NewState.ArrayIndex = 0;
					}
					else
					{
						//Index out of bound
						UE_LOGF(LogSerialization, Verbose, "Static array %ls has dimension of %d and trying to read element %d", *NewState.Property->GetFName().ToString(), NewState.Property->ArrayDim, CurrentState.ArrayIndex);
						Backend.SkipStructure();
						continue;
					}
					
				}
			}

			if (NewState.Property != nullptr)
			{
				// skip struct property if property filter is set and rejects it
				if (Policies.PropertyFilter && !Policies.PropertyFilter(NewState.Property, CurrentState.Property))
				{
					Backend.SkipStructure();
					continue;
				}

				NewState.TypeInfo = FindClass(NewState);

				StateStack.Push(CurrentState);
				CurrentState = NewState;
			}
			else
			{
				// error: structured property not found
				Backend.SkipStructure();

				if (Policies.MissingFields != EStructDeserializerErrorPolicies::Ignore)
				{
					UE_LOGF(LogSerialization, Verbose, "Structured property '%ls' not found", *PropertyName);
				}

				if (Policies.MissingFields == EStructDeserializerErrorPolicies::Error)
				{
					return false;
				}
			}
		}

		default:

			continue;
		}
	}

	// root structure not completed
	return false;
}

