// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"

namespace StructSerializationUtilities
{
	static bool IsLWCType(const UStruct* Type)
	{
		static TSet<FName> LWCTypes = {	NAME_Vector
										, NAME_Vector2D
										, NAME_Vector4
										, NAME_Matrix
										, NAME_Plane
										, NAME_Quat
										, NAME_Rotator
										, NAME_Transform
										, NAME_Box
										, NAME_Box2D
										, NAME_BoxSphereBounds
										, FName("OrientedBox")};
		if (Type)
		{
			const FName StructName = Type->GetFName();
			return LWCTypes.Contains(StructName);
		}

		return false;
	}
};
