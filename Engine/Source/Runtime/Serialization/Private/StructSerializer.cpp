// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructSerializer.h"
#include "UObject/UnrealType.h"
#include "UObject/PropertyOptional.h"
#include "IStructSerializerBackend.h"


/* Internal helpers
 *****************************************************************************/

namespace StructSerializer
{
	template <typename PropertyType>
	struct TScriptHelper_InContainer { };

	template <>
	struct TScriptHelper_InContainer<FArrayProperty> { using Type = FScriptArrayHelper_InContainer; };

	template <>
	struct TScriptHelper_InContainer<FSetProperty> { using Type = FScriptSetHelper_InContainer; };

	template <>
	struct TScriptHelper_InContainer<FMapProperty> { using Type = FScriptMapHelper_InContainer; };

	FStructSerializerState CreateItemState(FScriptArrayHelper& InHelper, FArrayProperty* InArrayProperty, int32 InValidatedInternalElementIndex, const FStructSerializerPolicies& Policies, EStructSerializerStateFlags InFlags)
	{
		return FStructSerializerState(InHelper.GetRawPtr(InValidatedInternalElementIndex), InArrayProperty->Inner, InFlags);
	};

	FStructSerializerState CreateItemState(FScriptMapHelper& InHelper, FMapProperty* InMapProperty, int32 InValidatedInternalElementIndex, const FStructSerializerPolicies& Policies, EStructSerializerStateFlags InFlags)
	{
		FStructSerializerState State = FStructSerializerState(InHelper.GetPairPtr(InValidatedInternalElementIndex), InMapProperty->ValueProp, InFlags);

		if (Policies.MapSerialization == EStructSerializerMapPolicies::KeyValuePair)
		{
			State.KeyData = InHelper.GetPairPtr(InValidatedInternalElementIndex);
			State.ValueData = State.KeyData;
			State.KeyProperty = InMapProperty->KeyProp;
		}

		return State;
	}

	FStructSerializerState CreateItemState(FScriptSetHelper& InHelper, FSetProperty* InSetProperty, int32 InValidatedInternalElementIndex, const FStructSerializerPolicies& Policies, EStructSerializerStateFlags InFlags)
	{
		return FStructSerializerState(InHelper.GetElementPtr(InValidatedInternalElementIndex), InSetProperty->ElementProp, InFlags);
	}

	bool IsArrayLike(FProperty* Property, const FStructSerializerPolicies& Policies)
	{
		return Property->IsA<FArrayProperty>()
			|| Property->IsA<FSetProperty>()
			|| (Property->IsA<FMapProperty>() && Policies.MapSerialization == EStructSerializerMapPolicies::Array);
	}

	void BeginIteratable(IStructSerializerBackend& Backend, FStructSerializerState& CurrentState, const FStructSerializerPolicies& Policies)
	{
		if (IsArrayLike(CurrentState.ValueProperty, Policies))
		{
			Backend.BeginArray(CurrentState);
		}
		else
		{
			Backend.BeginStructure(CurrentState);
		}
	}

	void EndIteratable(IStructSerializerBackend& Backend, FStructSerializerState& CurrentState, const FStructSerializerPolicies& Policies)
	{
		if (IsArrayLike(CurrentState.ValueProperty, Policies))
		{
			Backend.EndArray(CurrentState);
		}
		else
		{
			Backend.EndStructure(CurrentState);
		}
	}

	template <typename IteratablePropertyType>
	TArray<FStructSerializerState> GenerateStatesForIteratable(FStructSerializerState& CurrentState, const FStructSerializerPolicies& Policies)
	{
		using TScriptHelperType = typename TScriptHelper_InContainer<IteratablePropertyType>::Type;
		TArray<FStructSerializerState> OutStates;

		IteratablePropertyType* IteratableProperty = CastFieldChecked<IteratablePropertyType>(CurrentState.ValueProperty);
		TScriptHelperType ScriptHelper(IteratableProperty, CurrentState.ValueData);

		const int32 NumOutStates = CurrentState.ElementIndex != INDEX_NONE ? ScriptHelper.Num() : 1;
		OutStates.Reserve(NumOutStates);

		// If a specific index is asked only push that one on the stack
		if (CurrentState.ElementIndex != INDEX_NONE)
		{
			if (ScriptHelper.IsValidIndex(CurrentState.ElementIndex))
			{
				OutStates.Add(CreateItemState(ScriptHelper, IteratableProperty, CurrentState.ElementIndex, Policies, EStructSerializerStateFlags::WritingContainerElement));
			}
		}
		else
		{
			// push values on stack (in reverse order)
			for (int32 Index = ScriptHelper.Num() - 1; Index >= 0; --Index)
			{
				if (ScriptHelper.IsValidIndex(Index))
				{
					OutStates.Push(CreateItemState(ScriptHelper, IteratableProperty, Index, Policies, EStructSerializerStateFlags::None));
				}
			}
		}

		return OutStates;
	}

	template <typename IteratablePropertyType>
	void SerializeIterable(FStructSerializerState& CurrentState, TArray<FStructSerializerState>& StateStack, IStructSerializerBackend& Backend, const FStructSerializerPolicies& Policies)
	{
		if (!CurrentState.HasBeenProcessed)
		{
			// Only begin the iteratable if we are not serializing a single element.
			if (!EnumHasAnyFlags(CurrentState.StateFlags, EStructSerializerStateFlags::WritingContainerElement))
			{
				BeginIteratable(Backend, CurrentState, Policies);
			}

			if (!(CurrentState.ValueProperty->IsA<FArrayProperty>() && Backend.WritePODArray(CurrentState)))
			{
				CurrentState.HasBeenProcessed = true;
				StateStack.Push(CurrentState);

				// States are pushed on the stack in reverse order.
				for (FStructSerializerState& State : GenerateStatesForIteratable<IteratablePropertyType>(CurrentState, Policies))
				{
					StateStack.Push(MoveTemp(State));
				}
			}

		}
		else if (!EnumHasAnyFlags(CurrentState.StateFlags, EStructSerializerStateFlags::WritingContainerElement))
		{
			//Close iteratable only if we were not targeting a single element
			EndIteratable(Backend, CurrentState, Policies);
		}
	}

	void SerializeOptional(FStructSerializerState& CurrentState, TArray<FStructSerializerState>& StateStack, IStructSerializerBackend& Backend)
	{
		if (!CurrentState.HasBeenProcessed)
		{
			Backend.BeginStructure(CurrentState);

			CurrentState.HasBeenProcessed = true;
			StateStack.Push(CurrentState);

			const FOptionalProperty* OptionalProperty = CastFieldChecked<FOptionalProperty>(CurrentState.ValueProperty);
			FProperty* InnerProperty = OptionalProperty->GetValueProperty();
			const void* ValueData = CurrentState.ValueProperty->ContainerPtrToValuePtr<void>(CurrentState.ValueData);
			if (const void* InnerValue = OptionalProperty->GetValuePointerForReadIfSet(ValueData))
			{
				FStructSerializerState NewState;
				NewState.ValueData = InnerValue;
				NewState.ValueProperty = InnerProperty;
				NewState.FieldType = InnerProperty->GetClass();
				StateStack.Add(NewState);
			}
		}
		else
		{
			Backend.EndStructure(CurrentState);
		}
	}

	void SerializeStaticArray(FStructSerializerState& CurrentState, IStructSerializerBackend& Backend)
	{
		if (CurrentState.ElementIndex != INDEX_NONE)
		{
			if (CurrentState.ElementIndex < CurrentState.ValueProperty->ArrayDim)
			{
				Backend.WriteProperty(CurrentState, CurrentState.ElementIndex);
			}
		}
		else
		{
			Backend.BeginArray(CurrentState);
			for (int32 ArrayIndex = 0; ArrayIndex < CurrentState.ValueProperty->ArrayDim; ++ArrayIndex)
			{
				Backend.WriteProperty(CurrentState, ArrayIndex);
			}
			Backend.EndArray(CurrentState);
		}
	}

	TArray<FStructSerializerState> GenerateStructureStates(FStructSerializerState& CurrentState, const FStructSerializerPolicies& Policies)
	{
		TArray<FStructSerializerState> NewStates;

		const void* ValueData = CurrentState.ValueData;

		if (CurrentState.ValueType)
		{
			if (CurrentState.ValueProperty)
			{
				FFieldVariant Outer = CurrentState.ValueProperty->GetOwnerVariant();
				if ((Outer.ToField() == nullptr) || (Outer.ToField()->GetClass() != FArrayProperty::StaticClass()))
				{
					const int32 ContainerAddressIndex = CurrentState.ElementIndex != INDEX_NONE ? CurrentState.ElementIndex : 0;
					ValueData = CurrentState.ValueProperty->ContainerPtrToValuePtr<void>(CurrentState.ValueData, ContainerAddressIndex);
				}
			}

			for (TFieldIterator<FProperty> It(CurrentState.ValueType, EFieldIteratorFlags::IncludeSuper); It; ++It)
			{
				// Skip property if the filter function is set and rejects it.
				if (Policies.PropertyFilter && !Policies.PropertyFilter(*It, CurrentState.ValueProperty))
				{
					continue;
				}

				FStructSerializerState NewState;
				{
					NewState.ValueData = ValueData;
					NewState.ValueProperty = *It;
					NewState.FieldType = It->GetClass();
				}

				NewStates.Add(MoveTemp(NewState));
			}
		}

		return NewStates;
	}

	TArray<FStructSerializerState> GenerateStructureArrayStates(FStructSerializerState& CurrentState, const FStructSerializerPolicies& Policies)
	{
		TArray<FStructSerializerState> NewStates;

		// push elements on stack (in reverse order)
		for (int32 Index = CurrentState.ValueProperty->ArrayDim - 1; Index >= 0; --Index)
		{
			FStructSerializerState NewState;
			{
				NewState.ValueData = CurrentState.ValueData;
				NewState.ValueProperty = CurrentState.ValueProperty;
				NewState.ElementIndex = Index;
			}

			NewStates.Add(MoveTemp(NewState));
		}

		return NewStates;
	}

	UStruct* GetValueType(const FStructSerializerState& State)
	{
		UStruct* ValueType = nullptr;

		if (State.ValueProperty != nullptr)
		{
			//Get the type to iterate over the fields
			if (FStructProperty* StructProperty = CastField<FStructProperty>(State.ValueProperty))
			{
				ValueType = StructProperty->Struct;
			}
			else if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(State.ValueProperty))
			{
				ValueType = ObjectProperty->PropertyClass;
			}
		}

		return ValueType;
	}

	void SerializeStructInStaticArray(FStructSerializerState& CurrentState, TArray<FStructSerializerState>& StateStack, IStructSerializerBackend& Backend, const FStructSerializerPolicies& Policies)
	{
		if (!CurrentState.HasBeenProcessed)
		{
			//Push ourself to close the array
			CurrentState.HasBeenProcessed = true;
			StateStack.Push(CurrentState);

			Backend.BeginArray(CurrentState);

			for (FStructSerializerState& NewState : GenerateStructureArrayStates(CurrentState, Policies))
			{
				StateStack.Push(MoveTemp(NewState));
			}
		}
		else
		{
			Backend.EndArray(CurrentState);
		}
	}

	void SerializeStruct(FStructSerializerState& CurrentState, TArray<FStructSerializerState>& StateStack, IStructSerializerBackend& Backend, const FStructSerializerPolicies& Policies)
	{
		if (CurrentState.ValueProperty && CurrentState.ValueProperty->ArrayDim > 1 && CurrentState.ElementIndex == INDEX_NONE)
		{
			SerializeStructInStaticArray(CurrentState, StateStack, Backend, Policies);
		}
		else
		{
			if (!CurrentState.HasBeenProcessed)
			{
				if (UStruct* ValueType = GetValueType(CurrentState))
				{
					CurrentState.ValueType = ValueType;
				}

				if (CurrentState.ValueProperty)
				{
					Backend.BeginStructure(CurrentState);
				}

				CurrentState.HasBeenProcessed = true;
				StateStack.Push(CurrentState);

				TArray<FStructSerializerState> NewStates = GenerateStructureStates(CurrentState, Policies);

				// push child properties on stack (in reverse order)
				for (int32 Index = NewStates.Num() - 1; Index >= 0; --Index)
				{
					StateStack.Push(MoveTemp(NewStates[Index]));
				}
			}
			else
			{
				if (CurrentState.ValueProperty)
				{
					Backend.EndStructure(CurrentState);
				}
			}
		}
	}
	
	void Serialize(FStructSerializerState InitialState, IStructSerializerBackend& Backend, const FStructSerializerPolicies& Policies)
	{
		TArray<FStructSerializerState> StateStack;
		StateStack.Push(MoveTemp(InitialState));

		// Always encompass the element in an object
		Backend.BeginStructure(FStructSerializerState());


		// process state stack
		while (StateStack.Num() > 0)
		{
			FStructSerializerState CurrentState = StateStack.Pop(EAllowShrinking::No);

			// Structures
			if (!CurrentState.ValueProperty || CastField<FStructProperty>(CurrentState.ValueProperty))
			{
				SerializeStruct(CurrentState, StateStack, Backend, Policies);
			}

			// Dynamic arrays
			else if (CastField<FArrayProperty>(CurrentState.ValueProperty))
			{
				SerializeIterable<FArrayProperty>(CurrentState, StateStack, Backend, Policies);
			}

			// Maps
			else if (CastField<FMapProperty>(CurrentState.ValueProperty))
			{
				SerializeIterable<FMapProperty>(CurrentState, StateStack, Backend, Policies);
			}

			// Sets
			else if (CastField<FSetProperty>(CurrentState.ValueProperty))
			{
				SerializeIterable<FSetProperty>(CurrentState, StateStack, Backend, Policies);
			}

			// Optionals
			else if (CastField<FOptionalProperty>(CurrentState.ValueProperty))
			{
				SerializeOptional(CurrentState, StateStack, Backend);
			}

			// Static arrays
			else if (CurrentState.ValueProperty->ArrayDim > 1)
			{
				SerializeStaticArray(CurrentState, Backend);
			}

			// All other properties
			else
			{
				Backend.WriteProperty(CurrentState);
			}
		}

		Backend.EndStructure(FStructSerializerState());
	}
}

/* FStructSerializer static interface
 *****************************************************************************/

void FStructSerializer::Serialize(const void* Struct, UStruct& TypeInfo, IStructSerializerBackend& Backend, const FStructSerializerPolicies& Policies)
{
	check(Struct != nullptr);

	FStructSerializerState NewState;
	{
		NewState.ValueData = Struct;
		NewState.ValueType = &TypeInfo;
	}

	StructSerializer::Serialize(MoveTemp(NewState), Backend, Policies);
}

void FStructSerializer::SerializeElement(const void* Address, FProperty* Property, int32 ElementIndex, IStructSerializerBackend& Backend, const FStructSerializerPolicies& Policies)
{
	check(Address != nullptr);
	check(Property != nullptr);

	//Initial state with the desired property info
	FStructSerializerState NewState;
	{
		NewState.ValueData = Address;
		NewState.ElementIndex = ElementIndex;
		NewState.StateFlags = ElementIndex != INDEX_NONE ? EStructSerializerStateFlags::WritingContainerElement : EStructSerializerStateFlags::None;
		NewState.ValueProperty = Property;
		NewState.FieldType = Property->GetClass();
	}

	if (Policies.MapSerialization == EStructSerializerMapPolicies::KeyValuePair && ElementIndex != INDEX_NONE)
	{
		UE_LOGF(LogSerialization, Warning, "SerializeElement skipped map property %ls. Only supports maps as array.", *Property->GetFName().ToString());
		return;
	}

	StructSerializer::Serialize(MoveTemp(NewState), Backend, Policies);
}
