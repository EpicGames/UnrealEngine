// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Algo/ForEach.h"
#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Misc/AutomationTest.h"
#include "Templates/SubclassOf.h"
#include "Backends/JsonStructDeserializerBackend.h"
#include "Backends/JsonStructSerializerBackend.h"
#include "Backends/CborStructDeserializerBackend.h"
#include "Backends/CborStructSerializerBackend.h"
#include "StructDeserializer.h"
#include "StructSerializer.h"
#include "Tests/StructSerializerTestTypes.h"
#include "Tests/TestHarnessAdapter.h"
#include "UObject/CoreNet.h"

#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

/* Internal helpers
 *****************************************************************************/

namespace StructSerializerTest
{
	template<typename KeyType, typename ValueType>
	void CopyKeys(TMap<KeyType, ValueType>& OutMap, const TMap<KeyType, ValueType>& SourceMap)
	{
		OutMap.Empty(SourceMap.Num());
		Algo::ForEach(SourceMap, [&OutMap](const TPair<KeyType, ValueType>& Other)
		{
			OutMap.Add(Other.Key);
		});
	}

	template<>
	void CopyKeys(TMap<FString, FVector>& OutMap, const TMap<FString, FVector>& SourceMap)
	{
		OutMap.Empty(SourceMap.Num());
		Algo::ForEach(SourceMap, [&OutMap](const TPair<FString, FVector>& Other)
		{
			OutMap.Add(Other.Key, FVector(76.7f));
		});
	}

	template<>
	void CopyKeys(TMap<FString, FStructSerializerBuiltinTestStruct>& OutMap, const TMap<FString, FStructSerializerBuiltinTestStruct>& SourceMap)
	{
		OutMap.Empty(SourceMap.Num());
		Algo::ForEach(SourceMap, [&OutMap](const TPair<FString, FStructSerializerBuiltinTestStruct>& Other)
		{
			OutMap.Add(Other.Key, { NoInit });
		});
	}

	

	void ValidateNumerics(const FStructSerializerNumericTestStruct& Struct1, const FStructSerializerNumericTestStruct& Struct2)
	{
		CHECK_EQUALS(TEXT("Numerics.Int8 value must be the same before and after de-/serialization"), Struct1.Int8, Struct2.Int8);
		CHECK_EQUALS(TEXT("Numerics.Int16 value must be the same before and after de-/serialization"), Struct1.Int16, Struct2.Int16);
		CHECK_EQUALS(TEXT("Numerics.Int32 value must be the same before and after de-/serialization"), Struct1.Int32, Struct2.Int32);
		CHECK_EQUALS(TEXT("Numerics.Int64 value must be the same before and after de-/serialization"), Struct1.Int64, Struct2.Int64);
		CHECK_EQUALS(TEXT("Numerics.UInt8 value must be the same before and after de-/serialization"), Struct1.UInt8, Struct2.UInt8);
		CHECK_EQUALS(TEXT("Numerics.UInt16 value must be the same before and after de-/serialization"), Struct1.UInt16, Struct2.UInt16);
		CHECK_EQUALS(TEXT("Numerics.UInt32 value must be the same before and after de-/serialization"), Struct1.UInt32, Struct2.UInt32);
		CHECK_EQUALS(TEXT("Numerics.UInt64 value must be the same before and after de-/serialization"), Struct1.UInt64, Struct2.UInt64);
		CHECK_EQUALS(TEXT("Numerics.Float value must be the same before and after de-/serialization"), Struct1.Float, Struct2.Float);
		CHECK_EQUALS(TEXT("Numerics.Double value must be the same before and after de-/serialization"), Struct1.Double, Struct2.Double);
	}

	void ValidateBooleans(const FStructSerializerBooleanTestStruct& Struct1, const FStructSerializerBooleanTestStruct& Struct2)
	{
		CHECK_EQUALS(TEXT("Booleans.BoolFalse must be the same before and after de-/serialization"), Struct1.BoolFalse, Struct2.BoolFalse);
		CHECK_EQUALS(TEXT("Booleans.BoolTrue must be the same before and after de-/serialization"), Struct1.BoolTrue, Struct2.BoolTrue);
		CHECK_EQUALS(TEXT("Booleans.Bitfield0 must be the same before and after de-/serialization"), Struct1.Bitfield0, Struct2.Bitfield0);
		CHECK_EQUALS(TEXT("Booleans.Bitfield1 must be the same before and after de-/serialization"), Struct1.Bitfield1, Struct2.Bitfield1);
		CHECK_EQUALS(TEXT("Booleans.Bitfield2Set must be the same before and after de-/serialization"), Struct1.Bitfield2Set, Struct2.Bitfield2Set);
		CHECK_EQUALS(TEXT("Booleans.Bitfield3 must be the same before and after de-/serialization"), Struct1.Bitfield3, Struct2.Bitfield3);
		CHECK_EQUALS(TEXT("Booleans.Bitfield4Set must be the same before and after de-/serialization"), Struct1.Bitfield4Set, Struct2.Bitfield4Set);
		CHECK_EQUALS(TEXT("Booleans.Bitfield5Set must be the same before and after de-/serialization"), Struct1.Bitfield5Set, Struct2.Bitfield5Set);
		CHECK_EQUALS(TEXT("Booleans.Bitfield6 must be the same before and after de-/serialization"), Struct1.Bitfield6, Struct2.Bitfield6);
		CHECK_EQUALS(TEXT("Booleans.Bitfield7 must be the same before and after de-/serialization"), Struct1.Bitfield7Set, Struct2.Bitfield7Set);
	}

	void ValidateObjects(const FStructSerializerObjectTestStruct& Struct1, const FStructSerializerObjectTestStruct& Struct2)
	{
		CHECK_EQUALS(TEXT("Objects.RawClass must be the same before and after de-/serialization"), Struct1.RawClass, Struct2.RawClass);
		CHECK_EQUALS(TEXT("Objects.WrappedClass must be the same before and after de-/serialization"), Struct1.WrappedClass, Struct2.WrappedClass);
		CHECK_EQUALS(TEXT("Objects.SubClass must be the same before and after de-/serialization"), Struct1.SubClass, Struct2.SubClass);
		CHECK_EQUALS(TEXT("Objects.SoftClass must be the same before and after de-/serialization"), Struct1.SoftClass, Struct2.SoftClass);
		CHECK_EQUALS(TEXT("Objects.RawObject must be the same before and after de-/serialization"), Struct1.RawObject, Struct2.RawObject);
		CHECK_EQUALS(TEXT("Objects.WrappedObject must be the same before and after de-/serialization"), Struct1.WrappedObject, Struct2.WrappedObject);
		CHECK_EQUALS(TEXT("Objects.WeakObject must be the same before and after de-/serialization"), Struct1.WeakObject, Struct2.WeakObject);
		CHECK_EQUALS(TEXT("Objects.SoftObject must be the same before and after de-/serialization"), Struct1.SoftObject, Struct2.SoftObject);
		CHECK_EQUALS(TEXT("Objects.ClassPath must be the same before and after de-/serialization"), Struct1.ClassPath, Struct2.ClassPath);
		CHECK_EQUALS(TEXT("Objects.ObjectPath must be the same before and after de-/serialization"), Struct1.ObjectPath, Struct2.ObjectPath);
	}

	void ValidateBuiltIns(const FStructSerializerBuiltinTestStruct& Struct1, const FStructSerializerBuiltinTestStruct& Struct2)
	{
		CHECK_EQUALS(TEXT("Builtins.Guid must be the same before and after de-/serialization"), Struct1.Guid, Struct2.Guid);
		CHECK_EQUALS(TEXT("Builtins.Name must be the same before and after de-/serialization"), Struct1.Name, Struct2.Name);
		CHECK_EQUALS(TEXT("Builtins.String must be the same before and after de-/serialization"), Struct1.String, Struct2.String);
		CHECK_EQUALS(TEXT("Builtins.Text must be the same before and after de-/serialization"), Struct1.Text.ToString(), Struct2.Text.ToString());
		CHECK_EQUALS(TEXT("Builtins.Datetime must be the same before and after de-/serialization"), Struct1.Datetime, Struct2.Datetime);
		CHECK_EQUALS(TEXT("Builtins.Timespan must be the same before and after de-/serialization"), Struct1.Timespan, Struct2.Timespan);
		CHECK_EQUALS(TEXT("Builtins.Vector must be the same before and after de-/serialization"), Struct1.Vector, Struct2.Vector);
		CHECK_EQUALS(TEXT("Builtins.Vector4 must be the same before and after de-/serialization"), Struct1.Vector4, Struct2.Vector4);
		CHECK_EQUALS(TEXT("Builtins.Rotator must be the same before and after de-/serialization"), Struct1.Rotator, Struct2.Rotator);
		CHECK_EQUALS(TEXT("Builtins.Quat must be the same before and after de-/serialization"), Struct1.Quat, Struct2.Quat);
		CHECK_EQUALS(TEXT("Builtins.Color must be the same before and after de-/serialization"), Struct1.Color, Struct2.Color);
	}

	void ValidateArrays(const FStructSerializerArrayTestStruct& Struct1, const FStructSerializerArrayTestStruct& Struct2)
	{
		CHECK_EQUALS(TEXT("Arrays.Int32Array must be the same before and after de-/serialization"), Struct1.Int32Array, Struct2.Int32Array);
		CHECK_EQUALS(TEXT("Arrays.ByteArray must be the same before and after de-/serialization"), Struct1.ByteArray, Struct2.ByteArray);
		CHECK_EQUALS(TEXT("Arrays.StaticSingleElement[0] must be the same before and after de-/serialization"), Struct1.StaticSingleElement[0], Struct2.StaticSingleElement[0]);
		CHECK_EQUALS(TEXT("Arrays.StaticInt32Array[0] must be the same before and after de-/serialization"), Struct1.StaticInt32Array[0], Struct2.StaticInt32Array[0]);
		CHECK_EQUALS(TEXT("Arrays.StaticInt32Array[1] must be the same before and after de-/serialization"), Struct1.StaticInt32Array[1], Struct2.StaticInt32Array[1]);
		CHECK_EQUALS(TEXT("Arrays.StaticInt32Array[2] must be the same before and after de-/serialization"), Struct1.StaticInt32Array[2], Struct2.StaticInt32Array[2]);
		CHECK_EQUALS(TEXT("Arrays.StaticFloatArray[0] must be the same before and after de-/serialization"), Struct1.StaticFloatArray[0], Struct2.StaticFloatArray[0]);
		CHECK_EQUALS(TEXT("Arrays.StaticFloatArray[1] must be the same before and after de-/serialization"), Struct1.StaticFloatArray[1], Struct2.StaticFloatArray[1]);
		CHECK_EQUALS(TEXT("Arrays.StaticFloatArray[2] must be the same before and after de-/serialization"), Struct1.StaticFloatArray[2], Struct2.StaticFloatArray[2]);
		CHECK_EQUALS(TEXT("Arrays.VectorArray must be the same before and after de-/serialization"), Struct1.VectorArray, Struct2.VectorArray);
	}

	void ValidateMaps(const FStructSerializerMapTestStruct& Struct1, const FStructSerializerMapTestStruct& Struct2)
	{
		CHECK_MESSAGE(TEXT("Maps.IntToStr must be the same before and after de-/serialization"), Struct1.IntToStr.OrderIndependentCompareEqual(Struct2.IntToStr));
		CHECK_MESSAGE(TEXT("Maps.StrToStr must be the same before and after de-/serialization"), Struct1.StrToStr.OrderIndependentCompareEqual(Struct2.StrToStr));
		CHECK_MESSAGE(TEXT("Maps.StrToVec must be the same before and after de-/serialization"), Struct1.StrToVec.OrderIndependentCompareEqual(Struct2.StrToVec));
	}

	void ValidateSets(const FStructSerializerSetTestStruct& Struct1, const FStructSerializerSetTestStruct& Struct2)
	{
		CHECK_MESSAGE(TEXT("Sets.IntSet must be the same before and after de-/serialization"), Struct1.IntSet.Num() == Struct2.IntSet.Num() && Struct1.IntSet.Difference(Struct2.IntSet).Num() == 0);
		CHECK_MESSAGE(TEXT("Sets.StrSet must be the same before and after de-/serialization"), Struct1.StrSet.Num() == Struct2.StrSet.Num() && Struct1.StrSet.Difference(Struct2.StrSet).Num() == 0);
		CHECK_MESSAGE(TEXT("Sets.NameSet must be the same before and after de-/serialization"), Struct1.NameSet.Num() == Struct2.NameSet.Num() && Struct1.NameSet.Difference(Struct2.NameSet).Num() == 0);
		CHECK_MESSAGE(TEXT("Sets.StructSet must be the same before and after de-/serialization"), Struct1.StructSet.Num() == Struct2.StructSet.Num() && Struct1.StructSet.Difference(Struct2.StructSet).Num() == 0);
	}

	void ValidateOptionals(const FStructSerializerOptionalTestStruct& Struct1, const FStructSerializerOptionalTestStruct& Struct2)
	{
		CHECK_EQUALS(TEXT("Optionals.StrOptional must be the same before and after de-/serialization"), Struct1.StrOptional, Struct2.StrOptional);
		CHECK_EQUALS(TEXT("Optionals.StrOptionalUnset must be the same before and after de-/serialization"), Struct1.StrOptionalUnset, Struct2.StrOptionalUnset);
		CHECK_EQUALS(TEXT("Optionals.IntOptional must be the same before and after de-/serialization"), Struct1.IntOptional, Struct2.IntOptional);
		CHECK_EQUALS(TEXT("Optionals.IntOptionalUnset must be the same before and after de-/serialization"), Struct1.IntOptionalUnset, Struct2.IntOptionalUnset);
		CHECK_EQUALS(TEXT("Optionals.NameOptional must be the same before and after de-/serialization"), Struct1.NameOptional, Struct2.NameOptional);
		CHECK_EQUALS(TEXT("Optionals.NameOptionalUnset must be the same before and after de-/serialization"), Struct1.NameOptionalUnset, Struct2.NameOptionalUnset);
		CHECK_EQUALS(TEXT("Optionals.StructOptional must be the same before and after de-/serialization"), Struct1.StructOptional, Struct2.StructOptional);
		CHECK_EQUALS(TEXT("Optionals.StructOptionalUnset must be the same before and after de-/serialization"), Struct1.StructOptionalUnset, Struct2.StructOptionalUnset);
	}

	void ValidateLWCSerializationBackwardCompatibility(const FStructSerializerLWCTypesTest& Struct1, const FStructSerializerNonLWCTypesTest& Struct2)
	{
		//Make comparison by casting the double (lwc) version down to float (non-lwc) since this is what will happen during serialization
		CHECK_EQUALS(TEXT("LWC Vector must be deserialized to a Non-LWC Vector"), (FVector3f)Struct1.Vector, Struct2.Vector);
		CHECK_EQUALS(TEXT("LWC Vector2D must be deserialized to a Non-LWC Vector2D"), (FVector2f)Struct1.Vector2D, Struct2.Vector2D);
		CHECK_EQUALS(TEXT("LWC Vector4 must be deserialized to a Non-LWC Vector4"), (FVector4f)Struct1.Vector4, Struct2.Vector4);
		CHECK_EQUALS(TEXT("LWC Matrix must be deserialized to a Non-LWC Matrix"), (FMatrix44f)Struct1.Matrix, Struct2.Matrix);
		CHECK_EQUALS(TEXT("LWC Plane must be deserialized to a Non-LWC Plane"), (FPlane4f)Struct1.Plane, Struct2.Plane);
		CHECK_EQUALS(TEXT("LWC Quat must be deserialized to a Non-LWC Quat"), (FQuat4f)Struct1.Quat, Struct2.Quat);
		CHECK_EQUALS(TEXT("LWC Rotator must be deserialized to a Non-LWC Rotator"), (FRotator3f)Struct1.Rotator, Struct2.Rotator);
		CHECK_MESSAGE(TEXT("LWC Transform must be deserialized to a Non-LWC Transform"), Struct2.Transform.Equals((FTransform3f)Struct1.Transform));
		CHECK_EQUALS(TEXT("LWC Box must be deserialized to a Non-LWC Box"), (FBox3f)Struct1.Box, Struct2.Box);
		CHECK_EQUALS(TEXT("LWC Box2D must be deserialized to a Non-LWC Box2D"), (FBox2f)Struct1.Box2D, Struct2.Box2D);
		CHECK_EQUALS(TEXT("LWC BoxSphereBounds must be deserialized to a Non-LWC BoxSphereBounds"), (FBoxSphereBounds3f)Struct1.BoxSphereBounds, Struct2.BoxSphereBounds);
		CHECK_EQUALS(TEXT("LWC struct float must be the same when deserialized to a non-LWC struct float"), Struct1.Float, Struct2.Float);
		CHECK_EQUALS(TEXT("LWC struct double must be the same when deserialized to a non-LWC struct double"), Struct1.Double, Struct2.Double);

		const bool bIsOrientedBoxEqual = Struct1.OrientedBox.AxisX == FVector(Struct2.OrientedBox.AxisX)
			&& Struct1.OrientedBox.AxisY == FVector(Struct2.OrientedBox.AxisY)
			&& Struct1.OrientedBox.AxisZ == FVector(Struct2.OrientedBox.AxisZ)
			&& Struct1.OrientedBox.Center == FVector(Struct2.OrientedBox.Center)
			&& Struct1.OrientedBox.ExtentX == Struct2.OrientedBox.ExtentX
			&& Struct1.OrientedBox.ExtentY == Struct2.OrientedBox.ExtentY
			&& Struct1.OrientedBox.ExtentZ == Struct2.OrientedBox.ExtentZ;
		CHECK_MESSAGE(TEXT("LWC OrientedBox must be deserialized to a Non-LWC OrientedBox"), bIsOrientedBoxEqual);

		//Container testing
		bool bAreArrayEqual = Struct1.VectorArray.Num() == Struct2.VectorArray.Num();
		for (int32 Index = 0; Index < Struct1.VectorArray.Num() && bAreArrayEqual; ++Index)
		{
			if ((FVector3f)Struct1.VectorArray[Index] != Struct2.VectorArray[Index])
			{
				bAreArrayEqual = false;
			}
		}
		CHECK_MESSAGE(TEXT("Array of LWC Vectors must be deserialized to an Array of Non-LWC Vectors"), bAreArrayEqual);

		bool bAreMapsEqual = Struct1.StrToVec.Num() == Struct2.StrToVec.Num();
		for (const TPair<FString, FVector>& Struct1Pair : Struct1.StrToVec)
		{
			const FVector3f* Struct2Value = Struct2.StrToVec.Find(Struct1Pair.Key);
			if (Struct2Value == nullptr)
			{
				bAreMapsEqual = false;
				break;
			}

			const FVector3f Value = *Struct2Value;
			if (Value != (FVector3f)Struct1Pair.Value)
			{
				bAreMapsEqual = false;
				break;
			}
		}
		CHECK_MESSAGE(TEXT("Map of LWC Vectors must be deserialized to a Map of Non-LWC Vectors"), bAreMapsEqual);

		bool bAreSetsEqual = Struct1.VectorSet.Num() == Struct2.VectorSet.Num();
		for (const FVector& Vector : Struct1.VectorSet)
		{
			//Cast down like serialization has done
			const FVector3f FloatVector = (FVector3f)Vector;
			if (Struct2.VectorSet.Contains(FloatVector) == false)
			{
				bAreSetsEqual = false;
				break;
			}
		}
		CHECK_MESSAGE(TEXT("Set of LWC Vectors must be deserialized to a Set of Non-LWC Vectors"), bAreSetsEqual);
	}

	void ValidateLWCDeserializationBackwardCompatibility(const FStructSerializerNonLWCTypesTest& Struct1, const FStructSerializerLWCTypesTest& Struct2)
	{
		//Make comparison by casting the float (non-lwc) version up to double (lwc) since this is what will happen during deserialization
		CHECK_EQUALS(TEXT("Non-LWC Vector must be deserialized to a LWC Vector"), (FVector)Struct1.Vector, Struct2.Vector);
		CHECK_EQUALS(TEXT("Non-LWC Vector2D must be deserialized to a LWC Vector2D"), (FVector2D)Struct1.Vector2D, Struct2.Vector2D);
		CHECK_EQUALS(TEXT("Non-LWC Vector4 must be deserialized toa LWC Vector4"), (FVector4)Struct1.Vector4, Struct2.Vector4);
		CHECK_EQUALS(TEXT("Non-LWC Matrix must be deserialized to a LWC Matrix"), (FMatrix)Struct1.Matrix, Struct2.Matrix);
		CHECK_EQUALS(TEXT("Non-LWC Plane must be deserialized to a LWC Plane"), (FPlane)Struct1.Plane, Struct2.Plane);
		CHECK_EQUALS(TEXT("Non-LWC Quat must be deserialized to a LWC Quat"), (FQuat)Struct1.Quat, Struct2.Quat);
		CHECK_EQUALS(TEXT("Non-LWC Rotator must be deserialized to a LWC Rotator"), (FRotator)Struct1.Rotator, Struct2.Rotator);
		CHECK_MESSAGE(TEXT("Non-LWC Transform must be deserialized to a LWC Transform"), Struct2.Transform.Equals((FTransform)Struct1.Transform));
		CHECK_EQUALS(TEXT("Non-LWC Box must be deserialized to a LWC Box"), (FBox)Struct1.Box, Struct2.Box);

		CHECK_EQUALS(TEXT("Non-LWC Box2D must be deserialized to a LWC Box2D"), (FBox2D)Struct1.Box2D, Struct2.Box2D);
		CHECK_EQUALS(TEXT("Non-LWC BoxSphereBounds must be deserialized to a LWC BoxSphereBounds"), (FBoxSphereBounds)Struct1.BoxSphereBounds, Struct2.BoxSphereBounds);
		CHECK_EQUALS(TEXT("Non-LWC struct float must be the same when deserialized to a LWC struct float"), Struct1.Float, Struct2.Float);
		CHECK_EQUALS(TEXT("Non-LWC struct double must be the same when deserialized to a LWC struct double"), Struct1.Double, Struct2.Double);

		const bool bIsOrientedBoxEqual = FVector(Struct1.OrientedBox.AxisX) == Struct2.OrientedBox.AxisX
			&& FVector(Struct1.OrientedBox.AxisY) == Struct2.OrientedBox.AxisY
			&& FVector(Struct1.OrientedBox.AxisZ) == Struct2.OrientedBox.AxisZ
			&& FVector(Struct1.OrientedBox.Center) == Struct2.OrientedBox.Center
			&& Struct1.OrientedBox.ExtentX == Struct2.OrientedBox.ExtentX
			&& Struct1.OrientedBox.ExtentY == Struct2.OrientedBox.ExtentY
			&& Struct1.OrientedBox.ExtentZ == Struct2.OrientedBox.ExtentZ;
		CHECK_MESSAGE(TEXT("Non-LWC OrientedBox must be deserialized to a LWC OrientedBox"), bIsOrientedBoxEqual);

		//Container testing
		bool bAreArrayEqual = Struct1.VectorArray.Num() == Struct2.VectorArray.Num();
		for (int32 Index = 0; Index < Struct1.VectorArray.Num() && bAreArrayEqual; ++Index)
		{
			if ((FVector)Struct1.VectorArray[Index] != Struct2.VectorArray[Index])
			{
				bAreArrayEqual = false;
			}
		}
		CHECK_MESSAGE(TEXT("Array of Non-LWC Vectors must be deserialized to an Array of LWC Vectors"), bAreArrayEqual);

		bool bAreMapsEqual = Struct1.StrToVec.Num() == Struct2.StrToVec.Num();
		for (const TPair<FString, FVector3f>& Struct1Pair : Struct1.StrToVec)
		{
			const FVector* Struct2Value = Struct2.StrToVec.Find(Struct1Pair.Key);
			if (Struct2Value == nullptr)
			{
				bAreMapsEqual = false;
				break;
			}

			const FVector Value = *Struct2Value;
			if (Value != (FVector)Struct1Pair.Value)
			{
				bAreMapsEqual = false;
				break;
			}
		}
		CHECK_MESSAGE(TEXT("Map of Non-LWC Vectors must be deserialized to a Map of LWC Vectors"), bAreMapsEqual);

		bool bAreSetsEqual = Struct1.VectorSet.Num() == Struct2.VectorSet.Num();
		for (const FVector3f& Vector : Struct1.VectorSet)
		{
			//Cast down like serialization has done
			const FVector FloatVector = (FVector)Vector;
			if (Struct2.VectorSet.Contains(FloatVector) == false)
			{
				bAreSetsEqual = false;
				break;
			}
		}
		CHECK_MESSAGE(TEXT("Set of Non-LWC Vectors must be deserialized to a Set of LWC Vectors"), bAreSetsEqual);
	}

	void ValidateLWCTypes(const FStructSerializerLWCTypesTest& Struct1, const FStructSerializerLWCTypesTest& Struct2)
	{
		CHECK_EQUALS(TEXT("LWC Vector must be the same before and after de-serialization"), Struct1.Vector, Struct2.Vector);
		CHECK_EQUALS(TEXT("LWC Vector2D must be the same before and after de-serialization"), Struct1.Vector2D, Struct2.Vector2D);
		CHECK_EQUALS(TEXT("LWC Vector4 must be the same before and after de-serialization"), Struct1.Vector4, Struct2.Vector4);
		CHECK_EQUALS(TEXT("LWC Matrix must be the same before and after de-serialization"), Struct1.Matrix, Struct2.Matrix);
		CHECK_EQUALS(TEXT("LWC Plane must be the same before and after de-serialization"), Struct1.Plane, Struct2.Plane);
		CHECK_EQUALS(TEXT("LWC Quat must be the same before and after de-serialization"), Struct1.Quat, Struct2.Quat);
		CHECK_EQUALS(TEXT("LWC Rotator must be the same before and after de-serialization"), Struct1.Rotator, Struct2.Rotator);
		CHECK_MESSAGE(TEXT("LWC Transform must be the same before and after de-serialization"), Struct2.Transform.Equals(Struct1.Transform));
		CHECK_EQUALS(TEXT("LWC Box must be the same before and after de-serialization"), Struct1.Box, Struct2.Box);

		CHECK_EQUALS(TEXT("LWC Box2D must be the same before and after de-serialization"), Struct1.Box2D, Struct2.Box2D);
		CHECK_EQUALS(TEXT("LWC BoxSphereBounds must be the same before and after de-serialization"), Struct1.BoxSphereBounds, Struct2.BoxSphereBounds);
		CHECK_EQUALS(TEXT("LWC struct float must be the same before and after de-serialization"), Struct1.Float, Struct2.Float);
		CHECK_EQUALS(TEXT("LWC struct double must be the same before and after de-serialization"), Struct1.Double, Struct2.Double);

		const bool bIsOrientedBoxEqual = Struct1.OrientedBox.AxisX == Struct2.OrientedBox.AxisX
			&& Struct1.OrientedBox.AxisY == Struct2.OrientedBox.AxisY
			&& Struct1.OrientedBox.AxisZ == Struct2.OrientedBox.AxisZ
			&& Struct1.OrientedBox.Center == Struct2.OrientedBox.Center
			&& Struct1.OrientedBox.ExtentX == Struct2.OrientedBox.ExtentX
			&& Struct1.OrientedBox.ExtentY == Struct2.OrientedBox.ExtentY
			&& Struct1.OrientedBox.ExtentZ == Struct2.OrientedBox.ExtentZ;
		CHECK_MESSAGE(TEXT("LWC OrientedBox must be the same before and after de-serialization"), bIsOrientedBoxEqual);

		CHECK_EQUALS(TEXT("LWC test - Arrays.VectorArray must be the same before and after de-/serialization"), Struct1.VectorArray, Struct2.VectorArray);
		CHECK_MESSAGE(TEXT("LWC test - Maps.StrToVec must be the same before and after de-/serialization"), Struct1.StrToVec.OrderIndependentCompareEqual(Struct2.StrToVec));
		CHECK_MESSAGE(TEXT("LWC test - Sets.VectorSet must be the same before and after de-/serialization"), Struct1.VectorSet.Num() == Struct2.VectorSet.Num() && Struct1.VectorSet.Difference(Struct2.VectorSet).Num() == 0);
	}

	template<typename TSerializerBackend, typename TDeserializerBackend>
	void TestElementSerialization()
	{
		// serialization
	
		FStructSerializerTestStruct OriginalStruct;
		UClass* ObjectTestClass = UObjectTest::StaticClass();
		UObjectTest* ObjectTestObject = NewObject<UObjectTest>();
		// setup object tests
		OriginalStruct.Objects.RawClass = ObjectTestClass;
		OriginalStruct.Objects.WrappedClass = ObjectTestClass;
		OriginalStruct.Objects.SubClass = ObjectTestClass;
		OriginalStruct.Objects.SoftClass = ObjectTestClass;
		OriginalStruct.Objects.RawObject = ObjectTestObject;
		OriginalStruct.Objects.WrappedObject = ObjectTestObject;
		OriginalStruct.Objects.WeakObject = ObjectTestObject;
		OriginalStruct.Objects.SoftObject = ObjectTestObject;
		OriginalStruct.Objects.ClassPath = ObjectTestClass;
		OriginalStruct.Objects.ObjectPath = ObjectTestObject;

		{
			FStructSerializerPolicies Policies;
			Policies.MapSerialization = EStructSerializerMapPolicies::Array;

			FStructDeserializerPolicies DeserializerPolicies;
			DeserializerPolicies.MissingFields = EStructDeserializerErrorPolicies::Warning;
			DeserializerPolicies.MapPolicies = EStructDeserializerMapPolicies::Array;
			
			//Numerics
			{
				FStructSerializerTestStruct TestStruct = OriginalStruct;

				TArray<uint8> Buffer;
				FMemoryReader Reader(Buffer);
				FMemoryWriter Writer(Buffer);

				TSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
				TDeserializerBackend DeserializerBackend(Reader);

				const FName Member = GET_MEMBER_NAME_CHECKED(FStructSerializerTestStruct, Numerics);
				FProperty* Property = FindFProperty<FProperty>(FStructSerializerTestStruct::StaticStruct(), Member);
				FStructSerializer::SerializeElement(&TestStruct, Property, INDEX_NONE, SerializerBackend, Policies);

				FStructSerializerTestStruct TestStruct2(NoInit);
				CHECK_MESSAGE(TEXT("Deserialization must succeed"), FStructDeserializer::DeserializeElement(&TestStruct2, *FStructSerializerTestStruct::StaticStruct(), INDEX_NONE, DeserializerBackend, DeserializerPolicies));

				ValidateNumerics(TestStruct.Numerics, TestStruct2.Numerics);
			}

			//Booleans
			{
				TArray<uint8> Buffer;
				FMemoryReader Reader(Buffer);
				FMemoryWriter Writer(Buffer);
				
				TSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
				TDeserializerBackend DeserializerBackend(Reader);

				FStructSerializerTestStruct TestStruct = OriginalStruct;
				const FName Member = GET_MEMBER_NAME_CHECKED(FStructSerializerTestStruct, Booleans);
				FProperty* Property = FindFProperty<FProperty>(FStructSerializerTestStruct::StaticStruct(), Member);
				FStructSerializer::SerializeElement(&TestStruct, Property, INDEX_NONE, SerializerBackend, Policies);

				FStructSerializerTestStruct TestStruct2(NoInit);
				CHECK_MESSAGE(TEXT("Deserialization must succeed"), FStructDeserializer::DeserializeElement(&TestStruct2, *FStructSerializerTestStruct::StaticStruct(), INDEX_NONE, DeserializerBackend, DeserializerPolicies));

				ValidateBooleans(TestStruct.Booleans, TestStruct2.Booleans);
			}

			//Objects
			{
				TArray<uint8> Buffer;
				FMemoryReader Reader(Buffer);
				FMemoryWriter Writer(Buffer);

				TSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
				TDeserializerBackend DeserializerBackend(Reader);

				FStructSerializerTestStruct TestStruct = OriginalStruct;
				const FName Member = GET_MEMBER_NAME_CHECKED(FStructSerializerTestStruct, Objects);
				FProperty* Property = FindFProperty<FProperty>(FStructSerializerTestStruct::StaticStruct(), Member);
				FStructSerializer::SerializeElement(&TestStruct, Property, INDEX_NONE, SerializerBackend, Policies);

				FStructSerializerTestStruct TestStruct2(NoInit);
				CHECK_MESSAGE(TEXT("Deserialization must succeed"), FStructDeserializer::DeserializeElement(&TestStruct2, *FStructSerializerTestStruct::StaticStruct(), INDEX_NONE, DeserializerBackend, DeserializerPolicies));

				ValidateObjects(TestStruct.Objects, TestStruct2.Objects);
			}

			//Built ins
			{
				TArray<uint8> Buffer;
				FMemoryReader Reader(Buffer);
				FMemoryWriter Writer(Buffer);

				TSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
				TDeserializerBackend DeserializerBackend(Reader);

				FStructSerializerTestStruct TestStruct = OriginalStruct;
				const FName Member = GET_MEMBER_NAME_CHECKED(FStructSerializerTestStruct, Builtins);
				FProperty* Property = FindFProperty<FProperty>(FStructSerializerTestStruct::StaticStruct(), Member);
				FStructSerializer::SerializeElement(&TestStruct, Property, INDEX_NONE, SerializerBackend, Policies);

				FStructSerializerTestStruct TestStruct2(NoInit);
				CHECK_MESSAGE(TEXT("Deserialization must succeed"), FStructDeserializer::DeserializeElement(&TestStruct2, *FStructSerializerTestStruct::StaticStruct(), INDEX_NONE, DeserializerBackend, DeserializerPolicies));

				ValidateBuiltIns(TestStruct.Builtins, TestStruct2.Builtins);
			}

			//Arrays
			{
				TArray<uint8> Buffer;
				FMemoryReader Reader(Buffer);
				FMemoryWriter Writer(Buffer);

				TSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
				TDeserializerBackend DeserializerBackend(Reader);

				FStructSerializerTestStruct TestStruct = OriginalStruct;
				const FName Member = GET_MEMBER_NAME_CHECKED(FStructSerializerTestStruct, Arrays);
				FProperty* Property = FindFProperty<FProperty>(FStructSerializerTestStruct::StaticStruct(), Member);
				FStructSerializer::SerializeElement(&TestStruct, Property, INDEX_NONE, SerializerBackend, Policies);

				FStructSerializerTestStruct TestStruct2(NoInit);
				CHECK_MESSAGE(TEXT("Deserialization must succeed"), FStructDeserializer::DeserializeElement(&TestStruct2, *FStructSerializerTestStruct::StaticStruct(), INDEX_NONE, DeserializerBackend, DeserializerPolicies));

				ValidateArrays(TestStruct.Arrays, TestStruct2.Arrays);
			}

			//Maps
			{
				TArray<uint8> Buffer;
				FMemoryReader Reader(Buffer);
				FMemoryWriter Writer(Buffer);

				TSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
				TDeserializerBackend DeserializerBackend(Reader);

				FStructSerializerTestStruct TestStruct = OriginalStruct;
				const FName Member = GET_MEMBER_NAME_CHECKED(FStructSerializerTestStruct, Maps);
				FProperty* Property = FindFProperty<FProperty>(FStructSerializerTestStruct::StaticStruct(), Member);
				FStructSerializer::SerializeElement(&TestStruct, Property, INDEX_NONE, SerializerBackend, Policies);

				FStructSerializerTestStruct TestStruct2(NoInit);

				// Target Maps start off with keys copied as the map serialization policy is array and only deals with values. Leaves keys empty.
				CopyKeys(TestStruct2.Maps.IntToStr, TestStruct.Maps.IntToStr);
				CopyKeys(TestStruct2.Maps.StrToStr, TestStruct.Maps.StrToStr);
				CopyKeys(TestStruct2.Maps.StrToStruct, TestStruct.Maps.StrToStruct);
				CopyKeys(TestStruct2.Maps.StrToVec, TestStruct.Maps.StrToVec);

				CHECK_MESSAGE(TEXT("Deserialization must succeed"), FStructDeserializer::DeserializeElement(&TestStruct2, *FStructSerializerTestStruct::StaticStruct(), INDEX_NONE, DeserializerBackend, DeserializerPolicies));

				ValidateMaps(TestStruct.Maps, TestStruct2.Maps);
			}

			//Sets
			{
				TArray<uint8> Buffer;
				FMemoryReader Reader(Buffer);
				FMemoryWriter Writer(Buffer);

				TSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
				TDeserializerBackend DeserializerBackend(Reader);

				FStructSerializerTestStruct TestStruct = OriginalStruct;
				const FName Member = GET_MEMBER_NAME_CHECKED(FStructSerializerTestStruct, Sets);
				FProperty* Property = FindFProperty<FProperty>(FStructSerializerTestStruct::StaticStruct(), Member);
				FStructSerializer::SerializeElement(&TestStruct, Property, INDEX_NONE, SerializerBackend, Policies);

				FStructSerializerTestStruct TestStruct2(NoInit);
				CHECK_MESSAGE(TEXT("Deserialization must succeed"), FStructDeserializer::DeserializeElement(&TestStruct2, *FStructSerializerTestStruct::StaticStruct(), INDEX_NONE, DeserializerBackend, DeserializerPolicies));

				ValidateSets(TestStruct.Sets, TestStruct2.Sets);
			}

			//Optionals
			{
				TArray<uint8> Buffer;
				FMemoryReader Reader(Buffer);
				FMemoryWriter Writer(Buffer);

				TSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
				TDeserializerBackend DeserializerBackend(Reader);

				FStructSerializerTestStruct TestStruct = OriginalStruct;
				const FName Member = GET_MEMBER_NAME_CHECKED(FStructSerializerTestStruct, Optionals);
				FProperty* Property = FindFProperty<FProperty>(FStructSerializerTestStruct::StaticStruct(), Member);
				FStructSerializer::SerializeElement(&TestStruct, Property, INDEX_NONE, SerializerBackend, Policies);

				// Inputs:
				// - TestStruct: all the properties will be defaulted - "Unset" properties unset, and the rest set.
				// - TestStruct2: all the "Unset" properties will be set, and the rest will be unset.
				// Expected Output:
				// - TestStruct2: all the "Unset" properties will be unset, and the rest will be set.
				// Goal:
				// - Validate that unset properties are populated when data is available for deserialization.
				// - Validate that set properties are cleared when data isn't available for deserialization.
				FStructSerializerTestStruct TestStruct2(NoInit);
				TestStruct2.Optionals.StrOptionalUnset = TestStruct.Optionals.StrOptional;
				TestStruct2.Optionals.IntOptionalUnset = TestStruct.Optionals.IntOptional;
				TestStruct2.Optionals.NameOptionalUnset = TestStruct.Optionals.NameOptional;
				TestStruct2.Optionals.StructOptionalUnset = TestStruct.Optionals.StructOptional;

				CHECK_MESSAGE(TEXT("Deserialization must succeed"), FStructDeserializer::DeserializeElement(&TestStruct2, *FStructSerializerTestStruct::StaticStruct(), INDEX_NONE, DeserializerBackend, DeserializerPolicies));

				ValidateOptionals(TestStruct.Optionals, TestStruct2.Optionals);
			}

			//TArray<uint8> element
			{
				TArray<uint8> Buffer;
				FMemoryReader Reader(Buffer);
				FMemoryWriter Writer(Buffer);

				TSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
				TDeserializerBackend DeserializerBackend(Reader);

				FStructSerializerArrayTestStruct TestStruct = OriginalStruct.Arrays;
				FStructSerializerArrayTestStruct TestStruct2(NoInit);
				TestStruct2.ByteArray.SetNumUninitialized(TestStruct.ByteArray.Num());
				TestStruct2.ByteArray[0] = 89;
				TestStruct2.ByteArray[1] = 91;
				TestStruct2.ByteArray[2] = 93;

				const FName Member = GET_MEMBER_NAME_CHECKED(FStructSerializerArrayTestStruct, ByteArray);
				FProperty* Property = FindFProperty<FProperty>(FStructSerializerArrayTestStruct::StaticStruct(), Member);

				constexpr int32 TestIndex = 1;
				FStructSerializer::SerializeElement(&TestStruct, Property, TestIndex, SerializerBackend, Policies);
				CHECK_MESSAGE(TEXT("Deserialization must succeed"), FStructDeserializer::DeserializeElement(&TestStruct2, *FStructSerializerArrayTestStruct::StaticStruct(), TestIndex, DeserializerBackend, DeserializerPolicies));
				CHECK_NOT_EQUALS(TEXT("Arrays.ByteArray[0] must not be the same before and after de-/serialization of element 1"), TestStruct.ByteArray[0], TestStruct2.ByteArray[0]);
				CHECK_EQUALS(TEXT("Arrays.ByteArray[1] must be the same before and after de-/serialization of element 1"), TestStruct.ByteArray[TestIndex], TestStruct2.ByteArray[TestIndex]);
				CHECK_NOT_EQUALS(TEXT("Arrays.ByteArray[2] must not be the same before and after de-/serialization of element 1"), TestStruct.ByteArray[2], TestStruct2.ByteArray[2]);
			}

			//TArray<Struct> element
			{
				TArray<uint8> Buffer;
				FMemoryReader Reader(Buffer);
				FMemoryWriter Writer(Buffer);

				TSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
				TDeserializerBackend DeserializerBackend(Reader);

				FStructSerializerArrayTestStruct TestStruct = OriginalStruct.Arrays;
				FStructSerializerArrayTestStruct TestStruct2(NoInit);
				TestStruct2.StructArray.SetNumZeroed(TestStruct.StructArray.Num());

				const FName Member = GET_MEMBER_NAME_CHECKED(FStructSerializerArrayTestStruct, StructArray);
				FProperty* Property = FindFProperty<FProperty>(FStructSerializerArrayTestStruct::StaticStruct(), Member);

				constexpr int32 TestIndex = 1;
				FStructSerializer::SerializeElement(&TestStruct, Property, TestIndex, SerializerBackend, Policies);
				CHECK_MESSAGE(TEXT("Deserialization must succeed"), FStructDeserializer::DeserializeElement(&TestStruct2, *FStructSerializerArrayTestStruct::StaticStruct(), TestIndex, DeserializerBackend, DeserializerPolicies));
				CHECK_FALSE_MESSAGE(TEXT("Arrays.StructArray[0] must not be the same before and after de-/serialization of element 1"), TestStruct.StructArray[0] == TestStruct2.StructArray[0]);
				ValidateBuiltIns(TestStruct.StructArray[TestIndex], TestStruct2.StructArray[TestIndex]);
			}
			
			//static single element
			{
				TArray<uint8> Buffer;
				FMemoryReader Reader(Buffer);
				FMemoryWriter Writer(Buffer);

				TSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
				TDeserializerBackend DeserializerBackend(Reader);

				FStructSerializerArrayTestStruct TestStruct = OriginalStruct.Arrays;
				FStructSerializerArrayTestStruct TestStruct2(NoInit);
				TestStruct2.StaticSingleElement[0] = 998;

				const FName Member = GET_MEMBER_NAME_CHECKED(FStructSerializerArrayTestStruct, StaticSingleElement);
				FProperty* Property = FindFProperty<FProperty>(FStructSerializerArrayTestStruct::StaticStruct(), Member);

				constexpr int32 TestIndex = 0;
				FStructSerializer::SerializeElement(&TestStruct, Property, TestIndex, SerializerBackend, Policies);
				CHECK_MESSAGE(TEXT("Deserialization must succeed"), FStructDeserializer::DeserializeElement(&TestStruct2, *FStructSerializerArrayTestStruct::StaticStruct(), TestIndex, DeserializerBackend, DeserializerPolicies));
				CHECK_EQUALS(TEXT("Arrays.StaticSingleElement[0] must be the same before and after de-/serialization"), TestStruct.StaticSingleElement[TestIndex], TestStruct2.StaticSingleElement[TestIndex]);
			}

			//static float array element
			{
				TArray<uint8> Buffer;
				FMemoryReader Reader(Buffer);
				FMemoryWriter Writer(Buffer);

				TSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
				TDeserializerBackend DeserializerBackend(Reader);

				FStructSerializerArrayTestStruct TestStruct = OriginalStruct.Arrays;
				FStructSerializerArrayTestStruct TestStruct2(NoInit);
				FMemory::Memset(&TestStruct2.StaticFloatArray, 99, sizeof(TestStruct2.StaticFloatArray));

				const FName Member = GET_MEMBER_NAME_CHECKED(FStructSerializerArrayTestStruct, StaticFloatArray);
				FProperty* Property = FindFProperty<FProperty>(FStructSerializerArrayTestStruct::StaticStruct(), Member);

				constexpr int32 TestIndex = 1;
				FStructSerializer::SerializeElement(&TestStruct, Property, TestIndex, SerializerBackend, Policies);
				CHECK_MESSAGE(TEXT("Deserialization must succeed"), FStructDeserializer::DeserializeElement(&TestStruct2, *FStructSerializerArrayTestStruct::StaticStruct(), TestIndex, DeserializerBackend, DeserializerPolicies));
				CHECK_NOT_EQUALS(TEXT("Arrays.StaticFloatArray[0] must not be the same before and after de-/serialization of element 1"), TestStruct.StaticFloatArray[0], TestStruct2.StaticFloatArray[0]);
				CHECK_EQUALS(TEXT("Arrays.StaticFloatArray[1] must be the same before and after de-/serialization"), TestStruct.StaticFloatArray[TestIndex], TestStruct2.StaticFloatArray[TestIndex]);
				CHECK_NOT_EQUALS(TEXT("Arrays.StaticFloatArray[2] must not be the same before and after de-/serialization of element 1"), TestStruct.StaticFloatArray[2], TestStruct2.StaticFloatArray[2]);
			}

			//TMap<int32, FString> element
			{
				TArray<uint8> Buffer;
				FMemoryReader Reader(Buffer);
				FMemoryWriter Writer(Buffer);

				TSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
				TDeserializerBackend DeserializerBackend(Reader);

				FStructSerializerMapTestStruct TestStruct = OriginalStruct.Maps;
				FStructSerializerMapTestStruct TestStruct2(NoInit);
				CopyKeys(TestStruct2.IntToStr, TestStruct.IntToStr);

				const FName Member = GET_MEMBER_NAME_CHECKED(FStructSerializerMapTestStruct, IntToStr);
				FProperty* Property = FindFProperty<FProperty>(FStructSerializerMapTestStruct::StaticStruct(), Member);

				constexpr int32 TestIndex = 1;
				FStructSerializer::SerializeElement(&TestStruct, Property, TestIndex, SerializerBackend, Policies);
				CHECK_MESSAGE(TEXT("Deserialization must succeed"), FStructDeserializer::DeserializeElement(&TestStruct2, *FStructSerializerMapTestStruct::StaticStruct(), TestIndex, DeserializerBackend, DeserializerPolicies));

				TArray<int32> Keys;
				Keys.Reserve(TestStruct.IntToStr.Num());
				TestStruct.IntToStr.GenerateKeyArray(Keys);
				CHECK_NOT_EQUALS(TEXT("Maps.IntToStr[0] must not be the same before and after de-/serialization of element 1"), TestStruct.IntToStr[Keys[0]], TestStruct2.IntToStr[Keys[0]]);
				CHECK_EQUALS(TEXT("Maps.IntToStr[1] must be the same before and after de-/serialization of element 1"), TestStruct.IntToStr[Keys[TestIndex]], TestStruct2.IntToStr[Keys[TestIndex]]);
				CHECK_NOT_EQUALS(TEXT("Maps.IntToStr[2] must not be the same before and after de-/serialization of element 1"), TestStruct.IntToStr[Keys[2]], TestStruct2.IntToStr[Keys[2]]);
			}

			//TMap<FString, FVector> element
			{
				TArray<uint8> Buffer;
				FMemoryReader Reader(Buffer);
				FMemoryWriter Writer(Buffer);

				TSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
				TDeserializerBackend DeserializerBackend(Reader);

				FStructSerializerMapTestStruct TestStruct = OriginalStruct.Maps;
				FStructSerializerMapTestStruct TestStruct2(NoInit);
				CopyKeys(TestStruct2.StrToVec, TestStruct.StrToVec);

				const FName Member = GET_MEMBER_NAME_CHECKED(FStructSerializerMapTestStruct, StrToVec);
				FProperty* Property = FindFProperty<FProperty>(FStructSerializerMapTestStruct::StaticStruct(), Member);

				constexpr int32 TestIndex = 1;
				FStructSerializer::SerializeElement(&TestStruct, Property, TestIndex, SerializerBackend, Policies);
				CHECK_MESSAGE(TEXT("Deserialization must succeed"), FStructDeserializer::DeserializeElement(&TestStruct2, *FStructSerializerMapTestStruct::StaticStruct(), TestIndex, DeserializerBackend, DeserializerPolicies));

				TArray<FString> Keys;
				Keys.Reserve(TestStruct.IntToStr.Num());
				TestStruct.StrToVec.GenerateKeyArray(Keys);
				CHECK_NOT_EQUALS(TEXT("Maps.StrToVec[0] must not be the same before and after de-/serialization of element 1"), TestStruct.StrToVec[Keys[0]], TestStruct2.StrToVec[Keys[0]]);
				CHECK_EQUALS(TEXT("Maps.StrToVec[1] must be the same before and after de-/serialization of element 1"), TestStruct.StrToVec[Keys[TestIndex]], TestStruct2.StrToVec[Keys[TestIndex]]);
				CHECK_NOT_EQUALS(TEXT("Maps.StrToVec[2] must not be the same before and after de-/serialization of element 1"), TestStruct.StrToVec[Keys[2]], TestStruct2.StrToVec[Keys[2]]);
			}

			//TSet<FName> element
			{
				TArray<uint8> Buffer;
				FMemoryReader Reader(Buffer);
				FMemoryWriter Writer(Buffer);

				TSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
				TDeserializerBackend DeserializerBackend(Reader);

				FStructSerializerSetTestStruct TestStruct = OriginalStruct.Sets;
				FStructSerializerSetTestStruct TestStruct2(NoInit);
				TestStruct2.NameSet = { TEXT("Pre1"), TEXT("Pre2") , TEXT("Pre3") };

				const FName Member = GET_MEMBER_NAME_CHECKED(FStructSerializerSetTestStruct, NameSet);
				FProperty* Property = FindFProperty<FProperty>(FStructSerializerSetTestStruct::StaticStruct(), Member);

				constexpr int32 TestIndex = 1;
				FStructSerializer::SerializeElement(&TestStruct, Property, TestIndex, SerializerBackend, Policies);
				CHECK_MESSAGE(TEXT("Deserialization must succeed"), FStructDeserializer::DeserializeElement(&TestStruct2, *FStructSerializerSetTestStruct::StaticStruct(), TestIndex, DeserializerBackend, DeserializerPolicies));

				TArray<FName> SetArray1 = TestStruct.NameSet.Array();
				TArray<FName> SetArray2 = TestStruct2.NameSet.Array();
				CHECK_NOT_EQUALS(TEXT("Sets.NameSet[0] must not be the same before and after de-/serialization of element 1"), SetArray1[0], SetArray2[0]);
				CHECK_EQUALS(TEXT("Sets.NameSet[1] must be the same before and after de-/serialization of element 1"), SetArray1[TestIndex], SetArray2[TestIndex]);
				CHECK_NOT_EQUALS(TEXT("Sets.NameSet[2] must not be the same before and after de-/serialization of element 1"), SetArray1[2], SetArray2[2]);
			}
		}
	}

	void TestSerialization(IStructSerializerBackend& SerializerBackend, IStructDeserializerBackend& DeserializerBackend )
	{
		// serialization
		FStructSerializerTestStruct TestStruct;

		UClass* ObjectTestClass = UObjectTest::StaticClass();
		UObjectTest* ObjectTestObject = NewObject<UObjectTest>();
		// setup object tests
		TestStruct.Objects.RawClass = ObjectTestClass;
		TestStruct.Objects.WrappedClass = ObjectTestClass;
		TestStruct.Objects.SubClass = ObjectTestClass;
		TestStruct.Objects.SoftClass = ObjectTestClass;
		TestStruct.Objects.RawObject = ObjectTestObject;
		TestStruct.Objects.WrappedObject = ObjectTestObject;
		TestStruct.Objects.WeakObject = ObjectTestObject;
		TestStruct.Objects.SoftObject = ObjectTestObject;
		TestStruct.Objects.ClassPath = ObjectTestClass;
		TestStruct.Objects.ObjectPath = ObjectTestObject;

		{
			FStructSerializer::Serialize(TestStruct, SerializerBackend);
		}

		// deserialization
		FStructSerializerTestStruct TestStruct2(NoInit);
		{
			FStructDeserializerPolicies Policies;
			Policies.MissingFields = EStructDeserializerErrorPolicies::Warning;
			
			CHECK_MESSAGE(TEXT("Deserialization must succeed"), FStructDeserializer::Deserialize(TestStruct2, DeserializerBackend, Policies));
		}

		// test numerics
		ValidateNumerics(TestStruct.Numerics, TestStruct2.Numerics);

		// test booleans
		ValidateBooleans(TestStruct.Booleans, TestStruct2.Booleans);
	
		// test objects
		ValidateObjects(TestStruct.Objects, TestStruct2.Objects);

		// test built-ins
		ValidateBuiltIns(TestStruct.Builtins, TestStruct2.Builtins);

		// test arrays
		ValidateArrays(TestStruct.Arrays, TestStruct2.Arrays);

		// test maps
		ValidateMaps(TestStruct.Maps, TestStruct2.Maps);

		// test sets
		ValidateSets(TestStruct.Sets, TestStruct2.Sets);

		// test optionals
		ValidateOptionals(TestStruct.Optionals, TestStruct2.Optionals);

		//Test LWC types with standard de-serialization
		ValidateLWCTypes(TestStruct.LWCTypes, TestStruct2.LWCTypes);
	}

	void TestLWCSerialization(IStructSerializerBackend& SerializerBackend, IStructDeserializerBackend& DeserializerBackend)
	{
		// Serialization of LWC struct into non-LWC mode to mimick sending to older UE
		FStructSerializerLWCTypesTest TestLWCStruct;
		{
			FStructSerializer::Serialize(TestLWCStruct, SerializerBackend);
		}

		// Deserialization into non-LWC to mimick reception in an older UE
		FStructSerializerNonLWCTypesTest TestNonLWCStruct(NoInit);
		{
			FStructDeserializerPolicies Policies;
			Policies.MissingFields = EStructDeserializerErrorPolicies::Warning;

			CHECK_MESSAGE(TEXT("Deserialization must succeed"), FStructDeserializer::Deserialize(TestNonLWCStruct, DeserializerBackend, Policies));
		}

		ValidateLWCSerializationBackwardCompatibility(TestLWCStruct, TestNonLWCStruct);
	}

	void TestLWCDeserialization(IStructSerializerBackend& SerializerBackend, IStructDeserializerBackend& DeserializerBackend)
	{
		// Serialization of non lwc struct to mimick a struct coming from older UE
		FStructSerializerNonLWCTypesTest TestNonLWCStruct;
		{
			FStructSerializer::Serialize(TestNonLWCStruct, SerializerBackend);
		}

		// Deserialization into LWC type to mimick reception into a newer UE
		FStructSerializerLWCTypesTest TestLWCStruct(NoInit);
		{
			FStructDeserializerPolicies Policies;
			Policies.MissingFields = EStructDeserializerErrorPolicies::Warning;

			CHECK_MESSAGE(TEXT("Deserialization must succeed"), FStructDeserializer::Deserialize(TestLWCStruct, DeserializerBackend, Policies));
		}

		ValidateLWCDeserializationBackwardCompatibility(TestNonLWCStruct, TestLWCStruct);
	}
}


/* Tests
 *****************************************************************************/

TEST_CASE_NAMED(FStructSerializerTest, "System::Core::Serialization::StructSerializer", "[ApplicationContextMask][EngineFilter][StructSerializer]")
{
	const EStructSerializerBackendFlags BackendFlags = EStructSerializerBackendFlags::Default;

	// json
	{
		TArray<uint8> Buffer;
		FMemoryReader Reader(Buffer);
		FMemoryWriter Writer(Buffer);

		FJsonStructSerializerBackend SerializerBackend(Writer, BackendFlags);
		FJsonStructDeserializerBackend DeserializerBackend(Reader);

		StructSerializerTest::TestSerialization(SerializerBackend, DeserializerBackend);

		// uncomment this to look at the serialized data
		//GLog->Logf(TEXT("%s"), (TCHAR*)Buffer.GetData());
	}
	// cbor
	{
		TArray<uint8> Buffer;
		FMemoryReader Reader(Buffer);
		FMemoryWriter Writer(Buffer);

		FCborStructSerializerBackend SerializerBackend(Writer, BackendFlags);
		FCborStructDeserializerBackend DeserializerBackend(Reader);

		StructSerializerTest::TestSerialization(SerializerBackend, DeserializerBackend);
	}
	// cbor standard compliant endianness (big endian)
	{
		TArray<uint8> Buffer;
		FMemoryReader Reader(Buffer);
		FMemoryWriter Writer(Buffer);

		FCborStructSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default | EStructSerializerBackendFlags::WriteCborStandardEndianness);
		FCborStructDeserializerBackend DeserializerBackend(Reader, ECborEndianness::StandardCompliant);

		StructSerializerTest::TestSerialization(SerializerBackend, DeserializerBackend);
	}
	// cbor LWC (UE5) to NonLWC (UE4)
	{
		TArray<uint8> Buffer;
		FMemoryReader Reader(Buffer);
		FMemoryWriter Writer(Buffer);

		FCborStructSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::LegacyUE4 | EStructSerializerBackendFlags::WriteCborStandardEndianness);

		constexpr bool bIsLWCCompatibilityMode = false;
		FCborStructDeserializerBackend DeserializerBackend(Reader, ECborEndianness::StandardCompliant, bIsLWCCompatibilityMode);

		StructSerializerTest::TestLWCSerialization(SerializerBackend, DeserializerBackend);
	}
	// cbor Non LWC (UE4) to LWC (UE5)
	{
		TArray<uint8> Buffer;
		FMemoryReader Reader(Buffer);
		FMemoryWriter Writer(Buffer);

		FCborStructSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::LegacyUE4 | EStructSerializerBackendFlags::WriteCborStandardEndianness);

		constexpr bool bIsLWCCompatibilityMode = true;
		FCborStructDeserializerBackend DeserializerBackend(Reader, ECborEndianness::StandardCompliant, bIsLWCCompatibilityMode);

		StructSerializerTest::TestLWCDeserialization(SerializerBackend, DeserializerBackend);
	}
}

TEST_CASE_NAMED(FStructElementSerializerTest, "System::Core::Serialization::StructElementSerializer", "[ApplicationContextMask][EngineFilter][StructSerializer]")
{
	//Element de/serialization for both types of backend
	{
		StructSerializerTest::TestElementSerialization<FJsonStructSerializerBackend, FJsonStructDeserializerBackend>();
		StructSerializerTest::TestElementSerialization<FCborStructSerializerBackend, FCborStructDeserializerBackend>();
	}
}

TEST_CASE_NAMED(FStructSerializerCborByteArrayTest, "System::Core::Serialization::StructSerializerCborByteArray", "[ApplicationContextMask][EngineFilter][StructSerializer]")
{
	// Ensure TArray<uint8>/TArray<int8> are written as CBOR byte string (~2x more compact) by default rather than a CBOR array.
	{
		static_assert((EStructSerializerBackendFlags::Default & EStructSerializerBackendFlags::WriteByteArrayAsByteStream) == EStructSerializerBackendFlags::WriteByteArrayAsByteStream, "Test below expects 'EStructSerializerBackendFlags::Default' to contain 'EStructSerializerBackendFlags::WriteByteArrayAsByteStream'");
		
		// Serialization
		TArray<uint8> Buffer;
		FMemoryWriter Writer(Buffer);
		FCborStructSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
		FStructSerializerByteArray WrittenStruct;
		FStructSerializer::Serialize(WrittenStruct, SerializerBackend);

		CHECK_MESSAGE(TEXT("Arrays of int8/uint8 must be encoded in byte string (compact)"), Buffer.Num() == 54); // Copy the 54 bytes from VC++ Memory viewer to CBOR playground http://cbor.me/ to validate the count/content.

		// Deserialization
		FMemoryReader Reader(Buffer);
		FCborStructDeserializerBackend DeserializerBackend(Reader);
		FStructDeserializerPolicies Policies;
		Policies.MissingFields = EStructDeserializerErrorPolicies::Warning;
		FStructSerializerByteArray ReadStruct(NoInit);
		FStructDeserializer::Deserialize(ReadStruct, DeserializerBackend, Policies);

		CHECK_MESSAGE(TEXT("Value before TArray<uint8> must be the same before and after de-/serialization."), ReadStruct.Dummy1 == 1);
		CHECK_MESSAGE(TEXT("Value after TArray<uint8> must be the same before and after de-/serialization."), ReadStruct.Dummy2 == 2);
		CHECK_MESSAGE(TEXT("Value after TArray<int8> must be the same before and after de-/serialization."), ReadStruct.Dummy3 == 3);
		CHECK_MESSAGE(TEXT("Array uint8 must be the same before and after de-/serialization"), WrittenStruct.ByteArray == ReadStruct.ByteArray);
		CHECK_MESSAGE(TEXT("Array int8 must be the same before and after de-/serialization"), WrittenStruct.Int8Array == ReadStruct.Int8Array);
	}

	// Ensure TArray<uint8>/TArray<int8> encoded in CBOR byte string are skipped on deserialization if required by the policy.
	{
		// Serialization
		TArray<uint8> Buffer;
		FMemoryWriter Writer(Buffer);
		FCborStructSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
		FStructSerializerByteArray WrittenStruct;
		FStructSerializer::Serialize(WrittenStruct, SerializerBackend);

		// Deserialization
		FMemoryReader Reader(Buffer);
		FCborStructDeserializerBackend DeserializerBackend(Reader);

		// Skip the array properties named "ByteArray" and "Int8Array".
		FStructDeserializerPolicies Policies;
		Policies.PropertyFilter = [](const FProperty* CurrentProp, const FProperty* ParentProp)
		{
			const bool bFilteredOut = (CurrentProp->GetFName() == FName("ByteArray") || CurrentProp->GetFName() == FName("Int8Array"));
			return !bFilteredOut;
		};

		Policies.MissingFields = EStructDeserializerErrorPolicies::Warning;
		FStructSerializerByteArray ReadStruct(NoInit);
		FStructDeserializer::Deserialize(ReadStruct, DeserializerBackend, Policies);

		CHECK_MESSAGE(TEXT("Per deserializer policy, value before TArray<uint8> must be the same before and after de-/serialization."), ReadStruct.Dummy1 == 1);
		CHECK_MESSAGE(TEXT("Per deserializer policy, value after TArray<uint8> must be the same before and after de-/serialization."), ReadStruct.Dummy2 == 2);
		CHECK_MESSAGE(TEXT("Per deserializer policy, value after TArray<int8> must be the same before and after de-/serialization."), ReadStruct.Dummy3 == 3);
		CHECK_MESSAGE(TEXT("Per deserializer policy, TArray<uint8> must be skipped on deserialization"), ReadStruct.ByteArray.Num() == 0);
		CHECK_MESSAGE(TEXT("Per deserializer policy, TArray<int8> must be skipped on deserialization"), ReadStruct.Int8Array.Num() == 0);
	}

	// Ensure empty TArray<uint8>/TArray<int8> are written as zero-length CBOR byte string.
	{
		// Serialization
		TArray<uint8> Buffer;
		FMemoryWriter Writer(Buffer);
		FCborStructSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
		FStructSerializerByteArray WrittenStruct(NoInit); // Keep the TArray<> empty.
		WrittenStruct.Dummy1 = 1;
		WrittenStruct.Dummy2 = 2;
		WrittenStruct.Dummy3 = 3;
		FStructSerializer::Serialize(WrittenStruct, SerializerBackend);

		CHECK_MESSAGE(TEXT("Arrays of int8/uint8 must be encoded in byte string (compact)"), Buffer.Num() == 48); // Copy the 48 bytes from VC++ Memory viewer to CBOR playground http://cbor.me/ to validate the count/content.

		// Deserialization
		FMemoryReader Reader(Buffer);
		FCborStructDeserializerBackend DeserializerBackend(Reader);
		FStructDeserializerPolicies Policies;
		Policies.MissingFields = EStructDeserializerErrorPolicies::Warning;
		FStructSerializerByteArray ReadStruct(NoInit);
		FStructDeserializer::Deserialize(ReadStruct, DeserializerBackend, Policies);

		CHECK_MESSAGE(TEXT("Value before TArray<uint8> must be the same before and after de-/serialization."), ReadStruct.Dummy1 == 1);
		CHECK_MESSAGE(TEXT("Value after TArray<uint8> must be the same before and after de-/serialization."), ReadStruct.Dummy2 == 2);
		CHECK_MESSAGE(TEXT("Value after TArray<int8> must be the same before and after de-/serialization."), ReadStruct.Dummy3 == 3);
		CHECK_MESSAGE(TEXT("Array uint8 must be the same before and after de-/serialization"), WrittenStruct.ByteArray == ReadStruct.ByteArray);
		CHECK_MESSAGE(TEXT("Array int8 must be the same before and after de-/serialization"), WrittenStruct.Int8Array == ReadStruct.Int8Array);
	}

	// Ensure TArray<uint8>/TArray<int8> CBOR serialization is backward compatible. (Serializer can write the old format and deserializer can read it)
	{
		// Serialize TArray<uint8>/TArray<int8> it they were prior 4.25. (CBOR array rather than CBOR byte string)
		TArray<uint8> Buffer;
		FMemoryWriter Writer(Buffer);
		FCborStructSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::None);
		FStructSerializerByteArray WrittenStruct;
		FStructSerializer::Serialize(WrittenStruct, SerializerBackend);

		CHECK_MESSAGE(TEXT("Backward compatibility: Serialized size check"), Buffer.Num() == 60); // Copy the 60 bytes from VC++ Memory viewer to CBOR playground http://cbor.me/ to validate the count/content.

		// Deserialize TArray<uint8>/TArray<int8> as they were prior 4.25.
		FMemoryReader Reader(Buffer);
		FCborStructDeserializerBackend DeserializerBackend(Reader);
		FStructDeserializerPolicies Policies;
		Policies.MissingFields = EStructDeserializerErrorPolicies::Warning;
		FStructSerializerByteArray ReadStruct(NoInit);
		FStructDeserializer::Deserialize(ReadStruct, DeserializerBackend, Policies);

		CHECK_MESSAGE(TEXT("Backward compatibility: Integer must be the same before and after de-/serialization."), ReadStruct.Dummy1 == 1);
		CHECK_MESSAGE(TEXT("Backward compatibility: Integer must be the same before and after de-/serialization."), ReadStruct.Dummy2 == 2);
		CHECK_MESSAGE(TEXT("Backward compatibility: Integer must be the same before and after de-/serialization."), ReadStruct.Dummy3 == 3);
		CHECK_MESSAGE(TEXT("Backward compatibility: TArray<uint8> must be readable as CBOR array of number."), WrittenStruct.ByteArray == ReadStruct.ByteArray);
		CHECK_MESSAGE(TEXT("Backward compatibility: TArray<int8> must be readable as CBOR array of number."), WrittenStruct.Int8Array == ReadStruct.Int8Array);
	}
}


#endif // WITH_TESTS
