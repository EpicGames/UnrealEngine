// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"

#include "StructSerializerTestTypes.generated.h"

class UObject;

UCLASS()
class UObjectTest : public UObject
{
	GENERATED_BODY()
};

/**
 * Test structure for numeric properties.
 */
USTRUCT()
struct FStructSerializerNumericTestStruct
{
	GENERATED_BODY()

	UPROPERTY()
	int8 Int8;

	UPROPERTY()
	int16 Int16;

	UPROPERTY()
	int32 Int32;

	UPROPERTY()
	int64 Int64;

	UPROPERTY()
	uint8 UInt8;

	UPROPERTY()
	uint16 UInt16;

	UPROPERTY()
	uint32 UInt32;

	UPROPERTY()
	uint64 UInt64;

	UPROPERTY()
	float Float;

	UPROPERTY()
	double Double;

	/** Default constructor. */
	FStructSerializerNumericTestStruct()
		: Int8(-127)
		, Int16(-32767)
		, Int32(-2147483647)
		, Int64(-92233720368547/*75807*/)
		, UInt8(255)
		, UInt16(65535)
		, UInt32(4294967295)
		, UInt64(18446744073709/*551615*/)
		, Float(4.125)
		, Double(1.03125)
	{ }

	/** Creates an uninitialized instance. */
	FStructSerializerNumericTestStruct( ENoInit ) { }
};


/**
 * Test structure for boolean properties.
 */
USTRUCT()
struct FStructSerializerBooleanTestStruct
{
	GENERATED_BODY()

	UPROPERTY()
	bool BoolFalse;

	UPROPERTY()
	bool BoolTrue;

	UPROPERTY()
	uint8 Bitfield0 : 1;

	UPROPERTY()
	uint8 Bitfield1 : 1;

	UPROPERTY()
	uint8 Bitfield2Set : 1;

	UPROPERTY()
	uint8 Bitfield3 : 1;

	UPROPERTY()
	uint8 Bitfield4Set : 1;

	UPROPERTY()
	uint8 Bitfield5Set : 1;

	UPROPERTY()
	uint8 Bitfield6 : 1;

	UPROPERTY()
	uint8 Bitfield7Set : 1;

	/** Default constructor. */
	FStructSerializerBooleanTestStruct()
		: BoolFalse(false)
		, BoolTrue(true)
		, Bitfield0(0)
		, Bitfield1(0)
		, Bitfield2Set(1)
		, Bitfield3(0)
		, Bitfield4Set(1)
		, Bitfield5Set(1)
		, Bitfield6(0)
		, Bitfield7Set(1)
	{ }

	/** Creates an uninitialized instance. */
	FStructSerializerBooleanTestStruct( ENoInit ) { }
};


/**
 * Test structure for UObject properties.
 */
USTRUCT()
struct FStructSerializerObjectTestStruct
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UClass> RawClass;

	UPROPERTY()
	TObjectPtr<UClass> WrappedClass;

	UPROPERTY()
	TSubclassOf<UObjectTest> SubClass;

	UPROPERTY()
	TSoftClassPtr<UObjectTest> SoftClass;

	UPROPERTY()
	TObjectPtr<UObject> RawObject;

	UPROPERTY()
	TObjectPtr<UObject> WrappedObject;

	UPROPERTY()
	TWeakObjectPtr<UObjectTest> WeakObject;

	UPROPERTY()
	TSoftObjectPtr<UObjectTest> SoftObject;

	UPROPERTY()
	FSoftClassPath ClassPath;

	UPROPERTY()
	FSoftObjectPath ObjectPath;

	/** Default constructor. */
	FStructSerializerObjectTestStruct()
		: RawClass(nullptr)
		, WrappedClass(nullptr)
		, SubClass(nullptr)
		, SoftClass(nullptr)
		, RawObject(nullptr)
		, WrappedObject(nullptr)
		, WeakObject(nullptr)
		, SoftObject(nullptr)
		, ClassPath((UClass*)nullptr)
		, ObjectPath(nullptr)
	{}

	/** Creates an uninitialized instance. */
	FStructSerializerObjectTestStruct( ENoInit ) {}
};


/**
 * Test structure for properties of various built-in types.
 * @see NoExportTypes.h
 */
USTRUCT()
struct FStructSerializerBuiltinTestStruct
{
	GENERATED_BODY()

	UPROPERTY(meta=(IgnoreForMemberInitializationTest))
	FGuid Guid;

	UPROPERTY()
	FName Name;

	UPROPERTY()
	FString String;

	UPROPERTY()
	FText Text;

	UPROPERTY()
	FDateTime Datetime;

	UPROPERTY()
	FTimespan Timespan;

	UPROPERTY()
	FVector Vector;

	UPROPERTY()
	FVector4 Vector4;

	UPROPERTY()
	FRotator Rotator;

	UPROPERTY()
	FQuat Quat;

	UPROPERTY()
	FColor Color;

	/** Default constructor. */
	FStructSerializerBuiltinTestStruct()
		: Guid(FGuid::NewGuid())
		, Name(TEXT("Test FName"))
		, String("Test String")
		, Text(FText::FromString("Test Text"))
		, Datetime(2048, 2, 4, 8, 16, 32, 64)
		, Timespan(1, 2, 3, 4, 5)
		, Vector(1.0f, 2.0f, 3.0f)
		, Vector4(4.0f, 5.0f, 6.0f, 7.0f)
		, Rotator(4096, 8192, 16384)
		, Quat(1.0f, 2.0f, 3.0f, 0.46f)
		, Color(3, 255, 60, 255)
	{ }

	/** Creates an uninitialized instance. */
	FStructSerializerBuiltinTestStruct( ENoInit ) { }

	bool operator==(const FStructSerializerBuiltinTestStruct& Rhs) const
	{
		return Guid == Rhs.Guid && Name == Rhs.Name && String == Rhs.String && Text.EqualTo(Rhs.Text) && Datetime == Rhs.Datetime && Timespan == Rhs.Timespan && Vector == Rhs.Vector && Vector4 == Rhs.Vector4 && Rotator == Rhs.Rotator && Quat == Rhs.Quat && Color == Rhs.Color;
	}
};

/**
 * Test structure for LWC types.
 * @see NoExportTypes.h
 */
USTRUCT()
struct FStructSerializerLWCTypesTest
{
	GENERATED_BODY()

	UPROPERTY()
	FVector Vector;

	UPROPERTY()
	FVector2D Vector2D;

	UPROPERTY()
	FVector4 Vector4;

	UPROPERTY()
	FMatrix Matrix;

	UPROPERTY()
	FPlane Plane;
	
	UPROPERTY()
	FQuat Quat;
	
	UPROPERTY()
	FRotator Rotator;

	UPROPERTY()
	FTransform Transform;

	UPROPERTY()
	FBox Box;

	UPROPERTY()
	FBox2D Box2D;

	UPROPERTY()
	FBoxSphereBounds BoxSphereBounds;
	
	UPROPERTY()
	FOrientedBox OrientedBox;

	UPROPERTY()
	float Float;

	UPROPERTY()
	double Double;

	UPROPERTY()
	TArray<FVector> VectorArray;

	UPROPERTY()
	TMap<FString, FVector> StrToVec;

	UPROPERTY()
	TSet<FVector> VectorSet;

	/** Default constructor. */
	FStructSerializerLWCTypesTest()
		: Vector(1.25, 2.5, 3.75)
		, Vector2D(2.25, 3.5)
		, Vector4(0.5, 1.25, 2.5, 3.75)
		, Matrix(FVector(1.25, 2.25, 3.25), FVector(4.25, 5.5, 6.5), FVector(7.25, 8.25, 9.25), FVector(1, 2, 3))
		, Plane(0.25, 1.25, 2.25, 3.25)
		, Quat(0.25, 0.5, 0.75, 1.)
		, Rotator(2.25)
		, Transform(FQuat(1, 2, 3, 4))
		, Box(FVector4(1.0, 2.0, 3.0, 4.0), FVector4(5.0, 6.0, 7.0, 8.0))
		, Box2D(FVector2D(10.0, 20.0), FVector2D(30.0, 40.0))
		, BoxSphereBounds(FVector(1.0, 2.0, 3.0), FVector(10.0, 10.0, 10.0), 20.0)
		, Float(5.25f)
		, Double(1.114)
	{
		OrientedBox.AxisX = FVector(1.0, 1.0, 1.0);
		OrientedBox.AxisY = FVector(2.0, 2.0, 2.0);
		OrientedBox.AxisZ = FVector(3.0, 3.0, 3.0);
		OrientedBox.Center = FVector(5.0, 5.0, 5.0);
		OrientedBox.ExtentX = 10.0;
		OrientedBox.ExtentY = 20.0;
		OrientedBox.ExtentZ = 30.0;

		VectorArray.Add(FVector(1.0, 2.0, 3.0));
		VectorArray.Add(FVector(-1.0, -2.0, -3.0));

		StrToVec.Add(TEXT("V000"), FVector(0.0, 0.0, 0.0));
		StrToVec.Add(TEXT("V123"), FVector(1.0, 2.0, 3.0));
		StrToVec.Add(TEXT("V666"), FVector(6.0, 6.0, 6.0));

		VectorSet.Add(FVector(10.0, 11.0, 12.0));
	}

	/** Creates an uninitialized instance. */
	FStructSerializerLWCTypesTest(ENoInit) 
	{
	
	}
};

/** Float (Non LWC) version of FOrientedBox since the float version doesn't exist yet */
USTRUCT()
struct FOrientedBoxFloat
{
	GENERATED_BODY()

	FOrientedBoxFloat()
		: Center(0.0f)
		, AxisX(1.0f, 0.0f, 0.0f)
		, AxisY(0.0f, 1.0f, 0.0f)
		, AxisZ(0.0f, 0.0f, 1.0f)
		, ExtentX(1.0f)
		, ExtentY(1.0f)
		, ExtentZ(1.0f)
	{
	}

	/** Holds the center of the box. */
	UPROPERTY()
	FVector3f Center;

	/** Holds the x-axis vector of the box. Must be a unit vector. */
	UPROPERTY()
	FVector3f AxisX;

	/** Holds the y-axis vector of the box. Must be a unit vector. */
	UPROPERTY()
	FVector3f AxisY;

	/** Holds the z-axis vector of the box. Must be a unit vector. */
	UPROPERTY()
	FVector3f AxisZ;

	/** Holds the extent of the box along its x-axis. */
	UPROPERTY()
	float ExtentX;

	/** Holds the extent of the box along its y-axis. */
	UPROPERTY()
	float ExtentY;

	/** Holds the extent of the box along its z-axis. */
	UPROPERTY()
	float ExtentZ;
};

/**
 * Test structure for Non-LWC version of built in types.
 * @see NoExportTypes.h
 */
USTRUCT()
struct FStructSerializerNonLWCTypesTest
{
	GENERATED_BODY()

	UPROPERTY()
	FVector3f Vector;

	UPROPERTY()
	FVector2f Vector2D;

	UPROPERTY()
	FVector4f Vector4;

	UPROPERTY()
	FMatrix44f Matrix;

	UPROPERTY()
	FPlane4f Plane;

	UPROPERTY()
	FQuat4f Quat;

	UPROPERTY()
	FRotator3f Rotator;

	UPROPERTY()
	FTransform3f Transform;

	UPROPERTY()
	FBox3f Box;

	UPROPERTY()
	FBox2f Box2D;

	UPROPERTY()
	FBoxSphereBounds3f BoxSphereBounds;

	UPROPERTY()
	FOrientedBoxFloat OrientedBox;

	UPROPERTY()
	float Float;

	UPROPERTY()
	double Double;

	UPROPERTY()
	TArray<FVector3f> VectorArray;

	UPROPERTY()
	TMap<FString, FVector3f> StrToVec;

	UPROPERTY()
	TSet<FVector3f> VectorSet;

	/** Default constructor. */
	FStructSerializerNonLWCTypesTest()
		: Vector(1.25f, 2.5f, 3.75f)
		, Vector2D(2.25f, 3.5f)
		, Vector4(0.5f, 1.25f, 2.5f, 3.75f)
		, Matrix(FVector3f(1.25f, 2.25f, 3.25f), FVector3f(4.25f, 5.5f, 6.5f), FVector3f(7.25f, 8.25f, 9.25f), FVector3f(1.f, 2.f, 3.f))
		, Plane(0.25f, 1.25f, 2.25f, 3.25f)
		, Quat(0.25f, 0.5f, 0.75f, 1.f)
		, Rotator(2.25f)
		, Transform(FQuat4f(1.f, 2.f, 3.f, 4.f))
		, Box(FVector4f(1.0f, 2.0f, 3.0f, 4.0f), FVector4f(5.0f, 6.0f, 7.0f, 8.0f))
		, Box2D(FVector2f(10.0f, 20.0f), FVector2f(30.0f, 40.0f))
		, BoxSphereBounds(FVector3f(1.0f, 2.0f, 3.0f), FVector3f(10.0f, 10.0f, 10.0f), 20.0f)
		, Float(5.25f)
		, Double(1.114)
	{
		OrientedBox.AxisX = FVector3f(1.0f, 1.0f, 1.0f);
		OrientedBox.AxisY = FVector3f(2.0f, 2.0f, 2.0f);
		OrientedBox.AxisZ = FVector3f(3.0f, 3.0f, 3.0f);
		OrientedBox.Center = FVector3f(5.0f, 5.0f, 5.0f);
		OrientedBox.ExtentX = 10.0f;
		OrientedBox.ExtentY = 20.0f;
		OrientedBox.ExtentZ = 30.0f;

		VectorArray.Add(FVector3f(1.0f, 2.0f, 3.0f));
		VectorArray.Add(FVector3f(-1.0f, -2.0f, -3.0f));

		StrToVec.Add(TEXT("V000"), FVector3f(0.0f, 0.0f, 0.0f));
		StrToVec.Add(TEXT("V123"), FVector3f(1.0f, 2.0f, 3.0f));
		StrToVec.Add(TEXT("V666"), FVector3f(6.0f, 6.0f, 6.0f));

		VectorSet.Add(FVector3f(10.0f, 11.0f, 12.0f));
	}

	/** Creates an uninitialized instance. */
	FStructSerializerNonLWCTypesTest(ENoInit)
	{

	}
};

// basic type hash to test built in struct in sets
FORCEINLINE uint32 GetTypeHash(const FStructSerializerBuiltinTestStruct& S)
{
	return GetTypeHash(S.String);
}

/**
 * Test structure for byte array properties.
 */
USTRUCT()
struct FStructSerializerByteArray
{
	GENERATED_BODY()

	UPROPERTY()
	int32 Dummy1; // To test 'skip array'.

	UPROPERTY()
	TArray<uint8> ByteArray;

	UPROPERTY()
	int32 Dummy2; // To test 'skip array'.

	UPROPERTY()
	TArray<int8> Int8Array;

	UPROPERTY()
	int32 Dummy3; // To test 'skip array'.

	FStructSerializerByteArray()
	{
		Dummy1 = 1;
		Dummy2 = 2;
		Dummy3 = 3;

		ByteArray.Add(0);
		ByteArray.Add(127);
		ByteArray.Add(255);

		Int8Array.Add(-128);
		Int8Array.Add(0);
		Int8Array.Add(127);
	}

	FStructSerializerByteArray(ENoInit) { }
};


/**
 * Test structure for array properties.
 */
USTRUCT()
struct FStructSerializerArrayTestStruct
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<int32> Int32Array;

	UPROPERTY()
	TArray<uint8> ByteArray;

	UPROPERTY()
	int32 StaticSingleElement[1];

	UPROPERTY()
	int32 StaticInt32Array[3];

	UPROPERTY()
	float StaticFloatArray[3];

	UPROPERTY()
	TArray<FVector> VectorArray;

	UPROPERTY(meta=(IgnoreForMemberInitializationTest))
	TArray<FStructSerializerBuiltinTestStruct> StructArray;

	/** Default constructor. */
	FStructSerializerArrayTestStruct()
	{
		Int32Array.Add(-1);
		Int32Array.Add(0);
		Int32Array.Add(1);

		ByteArray.Add(0);
		ByteArray.Add(127);
		ByteArray.Add(255);

		StaticSingleElement[0] = 42;

		StaticInt32Array[0] = -1;
		StaticInt32Array[1] = 0;
		StaticInt32Array[2] = 1;

		StaticFloatArray[0] = -1.0f;
		StaticFloatArray[1] = 0.0f;
		StaticFloatArray[2] = 1.0f;

		VectorArray.Add(FVector(1.0f, 2.0f, 3.0f));
		VectorArray.Add(FVector(-1.0f, -2.0f, -3.0f));

		StructArray.AddDefaulted(2);
	}

	/** Creates an uninitialized instance. */
	FStructSerializerArrayTestStruct( ENoInit ) { }
};


/**
 * Test structure for map properties.
 */
USTRUCT()
struct FStructSerializerMapTestStruct
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<int32, FString> IntToStr;

	UPROPERTY()
	TMap<FString, FString> StrToStr;

	UPROPERTY()
	TMap<FString, FVector> StrToVec;

	UPROPERTY(meta=(IgnoreForMemberInitializationTest))
	TMap<FString, FStructSerializerBuiltinTestStruct> StrToStruct;

	/** Default constructor. */
	FStructSerializerMapTestStruct()
	{
		IntToStr.Add(1, TEXT("One"));
		IntToStr.Add(2, TEXT("Two"));
		IntToStr.Add(3, TEXT("Three"));

		StrToStr.Add(TEXT("StrAll"), TEXT("All"));
		StrToStr.Add(TEXT("StrYour"), TEXT("Your"));
		StrToStr.Add(TEXT("StrBase"), TEXT("Base"));

		StrToVec.Add(TEXT("V000"), FVector(0.0f, 0.0f, 0.0f));
		StrToVec.Add(TEXT("V123"), FVector(1.0f, 2.0f, 3.0f));
		StrToVec.Add(TEXT("V666"), FVector(6.0f, 6.0f, 6.0f));

		StrToStruct.Add(TEXT("StructOne"), FStructSerializerBuiltinTestStruct());
		StrToStruct.Add(TEXT("StructTwo"), FStructSerializerBuiltinTestStruct());
	}

	/** Creates an uninitialized instance. */
	FStructSerializerMapTestStruct( ENoInit ) { }
};

/**
 * Test structure for set properties.
 */
USTRUCT()
struct FStructSerializerSetTestStruct
{
	GENERATED_BODY()

	UPROPERTY()
	TSet<FString> StrSet;

	UPROPERTY()
	TSet<int32> IntSet;

	UPROPERTY()
	TSet<FName> NameSet;

	UPROPERTY(meta=(IgnoreForMemberInitializationTest))
	TSet<FStructSerializerBuiltinTestStruct> StructSet;

	/** Default constructor. */
	FStructSerializerSetTestStruct()
	{
		IntSet.Add(1);
		IntSet.Add(2);
		IntSet.Add(3);

		StrSet.Add(TEXT("Are"));
		StrSet.Add(TEXT("Belong"));
		StrSet.Add(TEXT("To"));
		StrSet.Add(TEXT("Us"));

		NameSet.Add(TEXT("Make"));
		NameSet.Add(TEXT("Your"));
		NameSet.Add(TEXT("Time"));

		StructSet.Add(FStructSerializerBuiltinTestStruct());
	}

	/** Creates an uninitialized instance. */
	FStructSerializerSetTestStruct( ENoInit ) { }
};

/**
 * Test structure for optional properties.
 */
USTRUCT()
struct FStructSerializerOptionalTestStruct
{
	GENERATED_BODY()

	UPROPERTY()
	TOptional<FString> StrOptional;

	UPROPERTY()
	TOptional<FString> StrOptionalUnset;

	UPROPERTY()
	TOptional<int32> IntOptional;

	UPROPERTY()
	TOptional<int32> IntOptionalUnset;

	UPROPERTY()
	TOptional<FName> NameOptional;

	UPROPERTY()
	TOptional<FName> NameOptionalUnset;

	UPROPERTY(meta=(IgnoreForMemberInitializationTest))
	TOptional<FStructSerializerBuiltinTestStruct> StructOptional;

	UPROPERTY()
	TOptional<FStructSerializerBuiltinTestStruct> StructOptionalUnset;

	// At the time of writing, optionals can not be used as keys/values in containers (optionals/arrays/sets/maps).

	// UPROPERTY()
	// TOptional<TOptional<int32>> OptionalOptionalInt;

	// UPROPERTY()
	// TArray<TOptional<int32>> ArrayOptionalInt;

	// UPROPERTY()
	// TSet<TOptional<int32>> SetOptionalInt;

	// UPROPERTY()
	// TMap<TOptional<int32>, TOptional<int32>> MapOptionalIntOptionalInt;

	/** Default constructor. */
	FStructSerializerOptionalTestStruct()
		: StrOptional(TEXT("Test String"))
		, IntOptional(1234)
		, NameOptional(TEXT("Test Name"))
		, StructOptional(FStructSerializerBuiltinTestStruct())
	{
	}

	/** Creates an uninitialized instance. */
	FStructSerializerOptionalTestStruct( ENoInit ) { }
};

/**
 * Test structure for all supported types.
 */
USTRUCT()
struct FStructSerializerTestStruct
{
	GENERATED_BODY()

	UPROPERTY()
	FStructSerializerNumericTestStruct Numerics;

	UPROPERTY()
	FStructSerializerBooleanTestStruct Booleans;

	UPROPERTY()
	FStructSerializerObjectTestStruct Objects;

	UPROPERTY(meta=(IgnoreForMemberInitializationTest))
	FStructSerializerBuiltinTestStruct Builtins;

	UPROPERTY()
	FStructSerializerArrayTestStruct Arrays;

	UPROPERTY()
	FStructSerializerMapTestStruct Maps;

	UPROPERTY()
	FStructSerializerSetTestStruct Sets;

	UPROPERTY()
	FStructSerializerOptionalTestStruct Optionals;

	UPROPERTY()
	FStructSerializerLWCTypesTest LWCTypes;

	/** Default constructor. */
	FStructSerializerTestStruct() = default;

	/** Creates an uninitialized instance. */
	FStructSerializerTestStruct( ENoInit )
		: Numerics(NoInit)
		, Booleans(NoInit)
		, Objects(NoInit)
		, Builtins(NoInit)
		, Arrays(NoInit)
		, Maps(NoInit)
		, Sets(NoInit)
		, Optionals(NoInit)
		, LWCTypes(NoInit)
	{ }
};
