// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CborReader.h"
#include "CborTypes.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "IStructDeserializerBackend.h"

class FArchive;
class FArrayProperty;
class FProperty;

/**
 * Implements a reader for UStruct deserialization using Cbor.
 */
class FCborStructDeserializerBackend
	: public IStructDeserializerBackend
{
public:

	/**
	 * Creates and initializes a new instance.
	 * @param Archive The archive to deserialize from.
	 * @param CborDataEndianness The CBOR data endianness stored in the archive.
	 * @note For backward compatibility and performance, the implementation default to the the platform endianness rather than the CBOR standard one (big endian).
	 */
	SERIALIZATION_API FCborStructDeserializerBackend(FArchive& Archive, ECborEndianness CborDataEndianness = ECborEndianness::Platform, bool bInIsLWCCompatibilityMode = false);
	SERIALIZATION_API virtual ~FCborStructDeserializerBackend();

public:

	// IStructDeserializerBackend interface
	SERIALIZATION_API virtual const FString& GetCurrentPropertyName() const override;
	SERIALIZATION_API virtual FString GetDebugString() const override;
	SERIALIZATION_API virtual const FString& GetLastErrorMessage() const override;
	SERIALIZATION_API virtual bool GetNextToken(EStructDeserializerBackendTokens& OutToken) override;
	SERIALIZATION_API virtual bool ReadProperty(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex) override;
	SERIALIZATION_API virtual bool ReadPODArray(FArrayProperty* ArrayProperty, void* Data) override;
	SERIALIZATION_API virtual void SkipArray() override;
	SERIALIZATION_API virtual void SkipStructure() override;

private:
	/** Holds the Cbor reader used for the actual reading of the archive. */
	FCborReader CborReader;

	/** Holds the last read Cbor Context. */
	FCborContext LastContext;

	/** Holds the last map key. */
	FString LastMapKey;

	/** The index of the next byte to copy from the CBOR byte stream into the corresponding TArray<uint8>/TArray<int8> property. */
	int32 DeserializingByteArrayIndex = 0;

	/** Whether a TArray<uint8>/TArray<int8> property is being deserialized. */
	bool bDeserializingByteArray = false;

	/** Whether we are deserializing for LWC backward compability mode. Incoming FloatProperties of LWC types deserialized into Double */
	bool bIsLWCCompatibilityMode = false;
};
