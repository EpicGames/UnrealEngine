// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CborWriter.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "IStructSerializerBackend.h"

class FArchive;

/**
 * Implements a writer for UStruct serialization using Cbor.
 */
class FCborStructSerializerBackend
	: public IStructSerializerBackend
{
public:

	/**
	 * Creates and initializes a new instance with the given flags.
	 *
	 * @param InArchive The archive to serialize into.
	 * @param InFlags The flags that control the serialization behavior (typically EStructSerializerBackendFlags::Default).
	 */
	SERIALIZATION_API FCborStructSerializerBackend(FArchive& InArchive, const EStructSerializerBackendFlags InFlags);

	SERIALIZATION_API virtual ~FCborStructSerializerBackend();

public:

	// IStructSerializerBackend interface
	SERIALIZATION_API virtual void BeginArray(const FStructSerializerState& State) override;
	SERIALIZATION_API virtual void BeginStructure(const FStructSerializerState& State) override;
	SERIALIZATION_API virtual void EndArray(const FStructSerializerState& State) override;
	SERIALIZATION_API virtual void EndStructure(const FStructSerializerState& State) override;
	SERIALIZATION_API virtual void WriteComment(const FString& Comment) override;
	SERIALIZATION_API virtual void WriteProperty(const FStructSerializerState& State, int32 ArrayIndex = 0) override;
	SERIALIZATION_API virtual bool WritePODArray(const FStructSerializerState& State) override;

private:
	/** Holds the Cbor writer used for the actual serialization. */
	FCborWriter CborWriter;

	/** Flags controlling the serialization behavior. */
	EStructSerializerBackendFlags Flags;

	/** Stores the accumulated bytes extracted from UByteProperty/UIntProperty when writing a TArray<uint8>/TArray<int8>. */
	TArray<uint8> AccumulatedBytes;

	/** Whether the serializer is encoding array of uint8/int8 */
	bool bSerializingByteArray = false;
};
