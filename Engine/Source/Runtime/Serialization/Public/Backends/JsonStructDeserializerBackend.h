// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "IStructDeserializerBackend.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonTypes.h"
#include "Templates/SharedPointer.h"

class FArchive;
class FProperty;

/**
 * Implements a reader for UStruct deserialization using Json.
 *
 * Note: The underlying Json de-serializer is currently hard-coded to use UCS2CHAR.
 * This is because the current JsonReader API does not allow writers to be substituted since it's
 * all based on templates. At some point we will refactor the low-level Json API to provide more
 * flexibility for serialization.
 */
class FJsonStructDeserializerBackend
	: public IStructDeserializerBackend
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param Archive The archive to deserialize from.
	 */
	FJsonStructDeserializerBackend( FArchive& Archive )
		: JsonReader(TJsonReader<WIDECHAR>::Create(&Archive))
	{ }

public:

	// IStructDeserializerBackend interface

	SERIALIZATION_API virtual const FString& GetCurrentPropertyName() const override;
	SERIALIZATION_API virtual FString GetDebugString() const override;
	SERIALIZATION_API virtual const FString& GetLastErrorMessage() const override;
	SERIALIZATION_API virtual bool GetNextToken( EStructDeserializerBackendTokens& OutToken ) override;
	SERIALIZATION_API virtual bool ReadProperty( FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex ) override;
	SERIALIZATION_API virtual void SkipArray() override;
	SERIALIZATION_API virtual void SkipStructure() override;

protected:
	FString& GetLastIdentifier()
	{
		return LastIdentifier;
	}

	EJsonNotation GetLastNotation()
	{
		return LastNotation;
	}

	TSharedRef<TJsonReader<WIDECHAR>>& GetReader()
	{
		return JsonReader;
	}

private:

	/** Holds the name of the last read Json identifier. */
	FString LastIdentifier;

	/** Holds the last read Json notation. */
	EJsonNotation LastNotation;

	/** Holds the Json reader used for the actual reading of the archive. */
	TSharedRef<TJsonReader<WIDECHAR>> JsonReader;
};
