// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "IStructSerializerBackend.h"
#include "Misc/EnumClassFlags.h"
#include "Serialization/JsonWriter.h"
#include "Templates/SharedPointer.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UnrealType.h"

class FArchive;

/**
 * Implements a writer for UStruct serialization using Json.
 *
 * Note: The underlying Json serializer is currently hard-coded to use UCS2CHAR and pretty-print.
 * This is because the current JsonWriter API does not allow writers to be substituted since it's
 * all based on templates. At some point we will refactor the low-level Json API to provide more
 * flexibility for serialization.
 */
class FJsonStructSerializerBackend
	: public IStructSerializerBackend
{
public:

	/**
	 * Creates and initializes a new instance with the given flags.
	 *
	 * @param InArchive The archive to serialize into.
	 * @param InFlags The flags that control the serialization behavior (typically EStructSerializerBackendFlags::Default).
	 */
	FJsonStructSerializerBackend( FArchive& InArchive, const EStructSerializerBackendFlags InFlags )
		: JsonWriter(TJsonWriter<UCS2CHAR>::Create(&InArchive))
		, Flags(InFlags)
	{ }

public:

	// IStructSerializerBackend interface

	SERIALIZATION_API virtual void BeginArray(const FStructSerializerState& State) override;
	SERIALIZATION_API virtual void BeginStructure(const FStructSerializerState& State) override;
	SERIALIZATION_API virtual void EndArray(const FStructSerializerState& State) override;
	SERIALIZATION_API virtual void EndStructure(const FStructSerializerState& State) override;
	SERIALIZATION_API virtual void WriteComment(const FString& Comment) override;
	SERIALIZATION_API virtual void WriteProperty(const FStructSerializerState& State, int32 ArrayIndex = 0) override;
	SERIALIZATION_API virtual bool WritePODArray(const FStructSerializerState& State) override;

protected:

	// Allow access to the internal JsonWriter to subclasses
	TSharedRef<TJsonWriter<UCS2CHAR>>& GetWriter()
	{
		return JsonWriter;
	}

	// Writes a property value to the serialization output.
	template<typename ValueType>
	void WritePropertyValue(const FStructSerializerState& State, const ValueType& Value)
	{
		//Write only value in case of no property or container elements
		if ((State.ValueProperty == nullptr) ||
			((State.ValueProperty->ArrayDim > 1
				|| State.ValueProperty->GetOwner<FArrayProperty>()
				|| State.ValueProperty->GetOwner<FSetProperty>()
				|| (State.ValueProperty->GetOwner<FMapProperty>() && State.KeyProperty == nullptr)) && !EnumHasAnyFlags(State.StateFlags, EStructSerializerStateFlags::WritingContainerElement)))
		{
			JsonWriter->WriteValue(Value);
		}
		//Write Key:Value in case of a map entry
		else if (State.KeyProperty != nullptr)
		{
			FString KeyString;
			State.KeyProperty->ExportTextItem_Direct(KeyString, State.KeyData, nullptr, nullptr, PPF_None);
			JsonWriter->WriteValue(KeyString, Value);
		}
		//Write PropertyName:Value for any other cases (single array element, single property, etc...)
		else
		{
			JsonWriter->WriteValue(State.ValueProperty->GetName(), Value);
		}
	}

	// Writes a null value to the serialization output.
	void WriteNull(const FStructSerializerState& State)
	{
		if ((State.ValueProperty == nullptr) ||
			((State.ValueProperty->ArrayDim > 1
				|| State.ValueProperty->GetOwner<FArrayProperty>()
				|| State.ValueProperty->GetOwner<FSetProperty>()
				|| (State.ValueProperty->GetOwner<FMapProperty>() && State.KeyProperty == nullptr)) && !EnumHasAnyFlags(State.StateFlags, EStructSerializerStateFlags::WritingContainerElement)))
		{
			JsonWriter->WriteNull();
		}
		else if (State.KeyProperty != nullptr)
		{
			FString KeyString;
			State.KeyProperty->ExportTextItem_Direct(KeyString, State.KeyData, nullptr, nullptr, PPF_None);
			JsonWriter->WriteNull(KeyString);
		}
		else
		{
			JsonWriter->WriteNull(State.ValueProperty->GetName());
		}
	}

private:

	/** Holds the Json writer used for the actual serialization. */
	TSharedRef<TJsonWriter<UCS2CHAR>> JsonWriter;

	/** Flags controlling the serialization behavior. */
	EStructSerializerBackendFlags Flags;
};
