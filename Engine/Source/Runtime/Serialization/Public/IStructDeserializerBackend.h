// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class Error;

/**
 * Enumerates serialization tokens.
 */
enum class EStructDeserializerBackendTokens
{
	/** End of an array. */
	ArrayEnd,

	/** Beginning of an array. */
	ArrayStart,

	/** A comment. */
	Comment,

	/** An error occurred when reading the token. */
	Error,

	/** No token available. */
	None,

	/** A scalar property. */
	Property,

	/** End of a data structure. */
	StructureEnd,

	/** Beginning of a data structure. */
	StructureStart,
};


/**
 * Interface for UStruct serializer backends.
 */
class IStructDeserializerBackend
{
public:

	/**
	 * Gets the identifier of the current field.
	 *
	 * @return The field's name.
	 */
	virtual const FString& GetCurrentPropertyName() const = 0;

	/**
	 * Gets a debug string for the reader's current state.
	 *
	 * The returned string contains debug information that is relevant to the reader's
	 * serialization format. For example, it could be a line and column number for text
	 * based formats, or a byte offset for binary serialization formats.
	 *
	 * @return The debug string.
	 */
	virtual FString GetDebugString() const = 0;

	/**
	 * Gets the last error message.
	 *
	 * @return Error message.
	 */
	virtual const FString& GetLastErrorMessage() const = 0;

	/**
	 * Reads the next token from the stream.
	 *
	 * @param OutToken Will contain the read token, if any.
	 * @return true if a token was read, false otherwise.
	 */
	virtual bool GetNextToken( EStructDeserializerBackendTokens& OutToken ) = 0;

	/**
	 * Reads the specified property from the stream.
	 *
	 * @param Property The property to read into.
	 * @param Outer The outer property holding the property to read (in case of arrays).
	 * @param Data The buffer that will hold the read data.
	 * @param ArrayIndex An index into the property array (for static arrays).
	 * @return true on success, false otherwise.
	 */
	virtual bool ReadProperty( FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex ) = 0;

	/**
	 * Reads the specified POD Array property from the stream.
	 * @note implementations will support only a Int8 or Byte array at the moment
	 * 
	 * @param ArrayProperty The array property to read into.
	 * @param Data The location of array to read the data into.
	 * @return true on success, false if need to fall back to per element array serialization.
	 */
	virtual bool ReadPODArray(FArrayProperty* ArrayProperty, void* Data) { return false;  };

	/** Skips the array that is currently being read from the stream. */
	virtual void SkipArray() = 0;

	/** Skips the object that is currently being read from the stream. */
	virtual void SkipStructure() = 0;

public:

	/** Virtual destructor. */
	virtual ~IStructDeserializerBackend() = default;
};
