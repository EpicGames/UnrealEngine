// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/Field.h"
#include "UObject/UnrealType.h"

/**
 * Flags controlling the behavior of struct serializer backends.
 */
enum class EStructSerializerBackendFlags
{
	/**
	 * No special behavior.
	 */
	None = 0,

	/**
	 * Write text in its complex exported format (eg, NSLOCTEXT(...)) rather than as a simple string.
	 * @note This is required to correctly support localization
	 */
	WriteTextAsComplexString = 1<<0,

	/**
	 * Write TArray<uint8>/TArray<int> as byte string if possible (CBOR), starting at 4.25.
	 */
	WriteByteArrayAsByteStream = 1<<1,

	/**
	 * Force the CBOR backend to write CBOR data in big endian (CBOR compliant endianness), available from 4.25. Caller must be opt-in.
	 * By default, the CBOR backend uses the endianness of the platform.
	 */
	WriteCborStandardEndianness = 1 << 2,

	/**
	 * Support backward compatibility for LWC types by writing double properties as floats.
	 */
	WriteLWCTypesAsFloats = 1 << 3,

	/**
	 * Legacy settings for backwards compatibility with code compiled for 4.25 up to UE5.
	 */
	LegacyUE4 = WriteTextAsComplexString | WriteByteArrayAsByteStream | WriteLWCTypesAsFloats,

	/**
	 * Default settings for code compiled for 5.0 onwards.
	 */
	Default = WriteTextAsComplexString | WriteByteArrayAsByteStream,
};
ENUM_CLASS_FLAGS(EStructSerializerBackendFlags);


/**
 * Flags related to the current state being serialized.
 */
enum class EStructSerializerStateFlags
{
	/**
	 * Nothing special.
	 */
	None = 0,

	/**
	 * Whether its serializing a single element from a container (array, set, map)
	 */
	 WritingContainerElement = 1 << 0,
};
ENUM_CLASS_FLAGS(EStructSerializerStateFlags);

/**
 * Structure for the write state stack.
 */
struct FStructSerializerState
{
	FStructSerializerState() = default;

	FStructSerializerState(void* InValuePtr, FProperty* InProperty, EStructSerializerStateFlags InFlags)
		: ValueData(InValuePtr)
		, ValueProperty(InProperty)
		, FieldType(InProperty->GetClass())
		, StateFlags(InFlags)
	{
	}

	/** Holds a flag indicating whether the property has been processed. */
	bool HasBeenProcessed = false;

	/** Holds a pointer to the key property's data. */
	const void* KeyData = nullptr;

	/** Holds the key property's meta data (only used for TMap). */
	FProperty* KeyProperty = nullptr;

	/** Holds a pointer to the property value's data. */
	const void* ValueData = nullptr;

	/** Holds the property value's meta data. */
	FProperty* ValueProperty = nullptr;

	/** Holds a pointer to the UStruct describing the data. */
	UStruct* ValueType = nullptr;

	/** Holds a pointer to the field type describing the data. */
	FFieldClass* FieldType = nullptr;

	/** Holds the element index that is targeted if an array/set/map */
	int32 ElementIndex = INDEX_NONE;

	/** Flags related for the current state */
	EStructSerializerStateFlags StateFlags = EStructSerializerStateFlags::None;
};


/**
 * Interface for UStruct serializer backends.
 */
class IStructSerializerBackend
{
public:

	/**
	 * Signals the beginning of an array.
	 *
	 * State.ValueProperty points to the property that holds the array.
	 *
	 * @param State The serializer's current state.
	 * @see BeginStructure, EndArray
	 */
	virtual void BeginArray(const FStructSerializerState& State) = 0;

	/**
	 * Signals the beginning of a child structure.
	 *
	 * State.ValueProperty points to the property that holds the struct.
	 *
	 * @param State The serializer's current state.
	 * @see BeginArray, EndStructure
	 */
	virtual void BeginStructure(const FStructSerializerState& State) = 0;

	/**
	 * Signals the end of an array.
	 *
	 * State.ValueProperty points to the property that holds the array.
	 *
	 * @param State The serializer's current state.
	 * @see BeginArray, EndStructure
	 */
	virtual void EndArray(const FStructSerializerState& State) = 0;

	/**
	 * Signals the end of an object.
	 *
	 * State.ValueProperty points to the property that holds the struct.
	 *
	 * @param State The serializer's current state.
	 * @see BeginStructure, EndArray
	 */
	virtual void EndStructure(const FStructSerializerState& State) = 0;

	/**
	 * Writes a comment to the output stream.
	 *
	 * @param Comment The comment text.
	 * @see BeginArray, BeginStructure, EndArray, EndStructure, WriteProperty
	 */
	virtual void WriteComment(const FString& Comment) = 0;

	/**
	 * Writes a property to the output stream.
	 *
	 * Depending on the context, properties to be written can be either object properties or array elements.
	 *
	 * State.KeyProperty points to the key property that holds the data to write.
	 * State.KeyData points to the key property's data.
	 * State.ValueProperty points to the property that holds the value to write.
	 * State.ValueData points to the actual data to write.
	 * State.TypeInfo contains the data's type information
	 * State.ArrayIndex is the optional index if the data is a value in an array.
	 *
	 * @param State The serializer's current state.
	 * @see BeginArray, BeginStructure, EndArray, EndStructure, WriteComment
	 */
	virtual void WriteProperty(const FStructSerializerState& State, int32 ArrayIndex = 0) = 0;

	/**
	 * Writes a POD Array property to the output stream.
	 * @note implementations will support only a Int8 or Byte array at the moment
	 * 
	 * State.ValueProperty points to the property that holds the value to write. needs to be an ArrayProperty with a properly supported InnerProperty.
	 * State.ValueData points to the actual data to write. The array itself in this case
	 * State.TypeInfo contains the data's type information
	 *
	 * @param State The serializer's current state.
	 * @return true if the array was properly written entirely as a pod array, false is we need to fallback to per element serialization
	 * @see BeginArray, BeginStructure, EndArray, EndStructure, WriteComment
	 */
	virtual bool WritePODArray(const FStructSerializerState& State) { return false; };

public:

	/** Virtual destructor. */
	virtual ~IStructSerializerBackend() = default;
};
