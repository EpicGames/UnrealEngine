// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Templates/Function.h"

class Error;
class FProperty;
class IStructDeserializerBackend;
class UStruct;

/**
 * Enumerates policies for various errors during de-serialization.
 */
enum class EStructDeserializerErrorPolicies
{
	/** Fail the de-serialization. */
	Error,
		
	/** Ignore the error (default). */
	Ignore,
		
	/** Print a warning to the log. */
	Warning,
};

/**
 * Enumerates policies for map de-serialization.
 */
enum class EStructDeserializerMapPolicies
{
	/** Deserialize maps as key-value pair - default. */
	KeyValuePair,

	/** Treat map properties as array using index. */
	Array,
};


/**
 * Structure for UStruct serialization policies.
 */
struct FStructDeserializerPolicies
{
	/** Holds the policy for handling missing fields. */
	EStructDeserializerErrorPolicies MissingFields;

	/** Policy about map property deserialization */
	EStructDeserializerMapPolicies MapPolicies;

	/** Predicate for performing advanced filtering of struct properties. 
		If set, the predicate should return true for all properties it wishes to include in the output.
	 */
	TFunction<bool (const FProperty* /*CurrentProp*/, const FProperty* /*ParentProp*/)> PropertyFilter;

	/** Default constructor. */
	FStructDeserializerPolicies()
		: MissingFields(EStructDeserializerErrorPolicies::Ignore)
		, MapPolicies(EStructDeserializerMapPolicies::KeyValuePair)
		, PropertyFilter()
	{ }
};


/**
 * Implements a static class that can deserialize UStruct based types.
 *
 * This class implements the basic functionality for the serialization of UStructs, such as
 * iterating a structure's properties and writing property values. The actual reading of
 * serialized input data is performed by de-serialization backends, which allows this
 * class to remain serialization format agnostic.
 */
class FStructDeserializer
{
public:

	/**
	 * Deserializes a data structure from an archive using the specified policy.
	 *
	 * @param OutStruct A pointer to the data structure to deserialize into.
	 * @param TypeInfo The data structure's type information.
	 * @param Backend The de-serialization backend to use.
	 * @param Policies The de-serialization policies to use.
	 * @return true if deserialization was successful, false otherwise.
	 */
	SERIALIZATION_API static bool Deserialize( void* OutStruct, UStruct& TypeInfo, IStructDeserializerBackend& Backend, const FStructDeserializerPolicies& Policies );
	
	/**
	 * Deserializes a data structure element from an archive using the specified policy.
	 *
	 * @param OutStruct A pointer to the data structure to deserialize into.
	 * @param TypeInfo The data structure's type information.
	 * @param ElementIndex The element index to deserialize in case of a container property.
	 * @param Backend The de-serialization backend to use.
	 * @param Policies The de-serialization policies to use.
	 * @return true if deserialization was successful, false otherwise.
	 */
	SERIALIZATION_API static bool DeserializeElement(void* OutStruct, UStruct& TypeInfo, int32 ElementIndex, IStructDeserializerBackend& Backend, const FStructDeserializerPolicies& Policies);

	/**
	 * Deserializes a data structure from an archive using the default policy.
	 *
	 * @param OutStruct A pointer to the data structure to deserialize into.
	 * @param TypeInfo The data structure's type information.
	 * @param Backend The de-serialization backend to use.
	 * @return true if deserialization was successful, false otherwise.
	 */
	static bool Deserialize( void* OutStruct, UStruct& TypeInfo, IStructDeserializerBackend& Backend )
	{
		return Deserialize(OutStruct, TypeInfo, Backend, FStructDeserializerPolicies());
	}

public:

	/**
	 * Deserializes a data structure from an archive using the default policy.
	 *
	 * @param OutStruct The struct to deserialize into.
	 * @param Backend The de-serialization backend to use.
	 * @return true if deserialization was successful, false otherwise.
	 */
	template<typename StructType>
	static bool Deserialize( StructType& OutStruct, IStructDeserializerBackend& Backend )
	{
		return Deserialize(&OutStruct, *StructType::StaticStruct(), Backend);
	}

	/**
	 * Deserializes a data structure from an archive using the specified policy.
	 *
	 * @param OutStruct The struct to deserialize into.
	 * @param Backend The de-serialization backend to use.
	 * @param Policies The de-serialization policies to use.
	 * @return true if deserialization was successful, false otherwise.
	 */
	template<typename StructType>
	static bool Deserialize( StructType& OutStruct, IStructDeserializerBackend& Backend, const FStructDeserializerPolicies& Policies )
	{
		return Deserialize(&OutStruct, *StructType::StaticStruct(), Backend, Policies);
	}
};
