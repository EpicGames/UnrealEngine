// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Templates/Function.h"

class Error;
class FProperty;
class IStructSerializerBackend;
class UStruct;

/**
 * Enumerates policies for serializing null values.
 */
enum class EStructSerializerNullValuePolicies
{
	/** Do not serialize NULL values. */
	Ignore,
		
	/** Serialize NULL values. */
	Serialize
};


/**
 * Enumerates policies for serializing object reference loops.
 */
enum class EStructSerializerReferenceLoopPolicies
{
	/** Circular references generate an error. */
	Error,
		
	/** Ignore circular references. */
	Ignore,
		
	/** Serialize circular references. */
	Serialize
};

/**
 * Enumerates policies for serializing map property type.
 */
enum class EStructSerializerMapPolicies
{
	/** Write map normally, as key value pair. */
	KeyValuePair,

	/** Ignore keys and write values as array. */
	Array,
};


/**
 * Structure for UStruct serialization policies.
 */
struct FStructSerializerPolicies
{
	/** Holds the policy for null values. */
	EStructSerializerNullValuePolicies NullValues;

	/** Holds the policy for reference loops. */
	EStructSerializerReferenceLoopPolicies ReferenceLoops;

	/** Policies dicting how maps are being written out */
	EStructSerializerMapPolicies MapSerialization;

	/** Predicate for performing advanced filtering of struct properties. 
		If set, the predicate should return true for all properties it wishes to include in the output.
	 */
	TFunction<bool (const FProperty* /*CurrentProp*/, const FProperty* /*ParentProp*/)> PropertyFilter;

	/** Default constructor. */
	FStructSerializerPolicies()
		: NullValues(EStructSerializerNullValuePolicies::Serialize)
		, ReferenceLoops(EStructSerializerReferenceLoopPolicies::Ignore)
		, MapSerialization(EStructSerializerMapPolicies::KeyValuePair)
		, PropertyFilter()
	{ }
};


/**
 * Implements a static class that can serialize UStruct based types.
 *
 * This class implements the basic functionality for the serialization of UStructs, such as
 * iterating a structure's properties and writing property values. The actual writing of
 * serialized output data is performed by serialization backends, which allows this class
 * to remain serialization format agnostic.
 *
 * The serializer's behavior can be customized with serialization policies. This allows for
 * control over how to handle null values, circular references and other edge cases.
 */
class FStructSerializer
{
public:

	/**
	 * Serializes a given data structure of the specified type using the specified policy.
	 *
	 * @param Struct The data structure to serialize.
	 * @param TypeInfo The structure's type information.
	 * @param Backend The serialization backend to use.
	 * @param Policies The serialization policies to use.
	 * @see Deserialize
	 */
	SERIALIZATION_API static void Serialize( const void* Struct, UStruct& TypeInfo, IStructSerializerBackend& Backend, const FStructSerializerPolicies& Policies );

	/**
	 * Serializes a given element of a data structure of the specified type using the specified policy.
	 *
	 * @param Address The address of the container of the property to serialize.
	 * @param Property The property to serialize.
	 * @param ElementIndex The index of the element to serialize if property is a container.
	 * @param Backend The serialization backend to use.
	 * @param Policies The serialization policies to use.
	 * @see Deserialize
	 */
	SERIALIZATION_API static void SerializeElement(const void* Address, FProperty* Property, int32 ElementIndex, IStructSerializerBackend& Backend, const FStructSerializerPolicies& Policies);


	/**
	 * Serializes a given data structure of the specified type using the default policy.
	 *
	 * @param Struct The data structure to serialize.
	 * @param TypeInfo The structure's type information.
	 * @param Backend The serialization backend to use.
	 * @see Deserialize
	 */
	static void Serialize( const void* Struct, UStruct& TypeInfo, IStructSerializerBackend& Backend )
	{
		Serialize(Struct, TypeInfo, Backend, FStructSerializerPolicies());
	}

public:

	/**
	 * Serializes a given USTRUCT to a string using the default policy.
	 *
	 * @param StructType The type of the struct to serialize.
	 * @param Struct The struct to serialize.
	 * @param Backend The serialization backend to use.
	 * @return A string holding the serialized object.
	 */
	template<typename StructType>
	static void Serialize( const StructType& Struct, IStructSerializerBackend& Backend )
	{
		Serialize(&Struct, *Struct.StaticStruct(), Backend, FStructSerializerPolicies());
	}

	/**
	 * Serializes a given USTRUCT to a string using the specified policy.
	 *
	 * @param StructType The type of the struct to serialize.
	 * @param Struct The struct to serialize.
	 * @param Backend The serialization backend to use.
	 * @param Policies The serialization policies to use.
	 * @return A string holding the serialized object.
	 */
	template<typename StructType>
	static void Serialize( const StructType& Struct, IStructSerializerBackend& Backend, const FStructSerializerPolicies& Policies )
	{
		Serialize(&Struct, *Struct.StaticStruct(), Backend, Policies);
	}
};
